// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyPathHelpers.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyPathHelpers)

/** Internal helper functions */
namespace PropertyPathHelpersInternal
{
	template<typename ContainerType>
	static bool IteratePropertyPathRecursive(UStruct* InStruct, ContainerType* InContainer, int32 SegmentIndex, const FCachedPropertyPath& InPropertyPath, FPropertyPathResolver& InResolver)
	{
		const FPropertyPathSegment& Segment = InPropertyPath.GetSegment(SegmentIndex);
		const int32 ArrayIndex = Segment.GetArrayIndex() == INDEX_NONE ? 0 : Segment.GetArrayIndex();

		// Reset cached address usage flag at the path root. This will be reset later in the recursion if conditions are not met in the path.
		if(SegmentIndex == 0)
		{
			InPropertyPath.SetCachedContainer(InContainer);
			InPropertyPath.SetCanSafelyUsedCachedAddress(true);
		}

		// Obtain the property info from the given structure definition
		FFieldVariant Field = Segment.Resolve(InStruct);
		if (Field.IsValid())
		{
			const bool bFinalSegment = SegmentIndex == (InPropertyPath.GetNumSegments() - 1);

			if ( FProperty* Property = Field.Get<FProperty>() )
			{
				if (bFinalSegment)
				{
					return InResolver.Resolve(static_cast<ContainerType*>(InContainer), InPropertyPath);
				}
				else
				{
					// Check first to see if this is a simple object (eg. not an array of objects)
					if ( FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property) )
					{
						// Object boundary that can change, so we cant use the cached address safely
						InPropertyPath.SetCanSafelyUsedCachedAddress(false);

						// If it's an object we need to get the value of the property in the container first before we 
						// can continue, if the object is null we safely stop processing the chain of properties.
						if ( UObject* CurrentObject = ObjectProperty->GetPropertyValue_InContainer(InContainer, ArrayIndex) )
						{
							InPropertyPath.SetCachedLastContainer(CurrentObject, SegmentIndex);
							return IteratePropertyPathRecursive(CurrentObject->GetClass(), CurrentObject, SegmentIndex + 1, InPropertyPath, InResolver);
						}
					}
					// Check to see if this is a simple weak object property (eg. not an array of weak objects).
					if ( FWeakObjectProperty* WeakObjectProperty = CastField<FWeakObjectProperty>(Property) )
					{
						// Object boundary that can change, so we cant use the cached address safely
						InPropertyPath.SetCanSafelyUsedCachedAddress(false);

						FWeakObjectPtr WeakObject = WeakObjectProperty->GetPropertyValue_InContainer(InContainer, ArrayIndex);

						// If it's an object we need to get the value of the property in the container first before we 
						// can continue, if the object is null we safely stop processing the chain of properties.
						if ( UObject* CurrentObject = WeakObject.Get() )
						{
							InPropertyPath.SetCachedLastContainer(CurrentObject, SegmentIndex);
							return IteratePropertyPathRecursive(CurrentObject->GetClass(), CurrentObject, SegmentIndex + 1, InPropertyPath, InResolver);
						}
					}
					// Check to see if this is a simple soft object property (eg. not an array of soft objects).
					else if ( FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(Property) )
					{
						// Object boundary that can change, so we cant use the cached address safely
						InPropertyPath.SetCanSafelyUsedCachedAddress(false);

						FSoftObjectPtr SoftObject = SoftObjectProperty->GetPropertyValue_InContainer(InContainer, ArrayIndex);

						// If it's an object we need to get the value of the property in the container first before we 
						// can continue, if the object is null we safely stop processing the chain of properties.
						if ( UObject* CurrentObject = SoftObject.Get() )
						{
							InPropertyPath.SetCachedLastContainer(CurrentObject, SegmentIndex);
							return IteratePropertyPathRecursive(CurrentObject->GetClass(), CurrentObject, SegmentIndex + 1, InPropertyPath, InResolver);
						}
					}
					// Check to see if this is a simple structure (eg. not an array of structures)
					else if ( FStructProperty* StructProp = CastField<FStructProperty>(Property) )
					{
						// Recursively call back into this function with the structure property and container value
						return IteratePropertyPathRecursive<void>(StructProp->Struct, StructProp->ContainerPtrToValuePtr<void>(InContainer, ArrayIndex), SegmentIndex + 1, InPropertyPath, InResolver);
					}
					else if ( FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property) )
					{
						// Dynamic array boundary that can change, so we cant use the cached address safely
						InPropertyPath.SetCanSafelyUsedCachedAddress(false);

						// It is an array, now check to see if this is an array of structures
						if ( FStructProperty* ArrayOfStructsProp = CastField<FStructProperty>(ArrayProp->Inner) )
						{
							FScriptArrayHelper_InContainer ArrayHelper(ArrayProp, InContainer);
							if ( ArrayHelper.IsValidIndex(ArrayIndex) )
							{
								// Recursively call back into this function with the array element and container value
								return IteratePropertyPathRecursive<void>(ArrayOfStructsProp->Struct, static_cast<void*>(ArrayHelper.GetRawPtr(ArrayIndex)), SegmentIndex + 1, InPropertyPath, InResolver);
							}
						}
						// if it's not an array of structs, maybe it's an array of classes
						//else if ( FObjectProperty* ObjectProperty = CastField<FObjectProperty>(ArrayProp->Inner) )
						{
							//TODO Add support for arrays of objects.
						}
					}
					else if( FSetProperty* SetProperty = CastField<FSetProperty>(Property) )
					{
						// TODO: we dont support set properties yet
					}
					else if( FMapProperty* MapProperty = CastField<FMapProperty>(Property) )
					{
						// TODO: we dont support map properties yet
					}
				}
			}
			else
			{
				// If it's the final segment, use the resolver to get the value.
				if (bFinalSegment)
				{
					return InResolver.Resolve(static_cast<ContainerType*>(InContainer), InPropertyPath);
				}
				else
				{
					// If it's not the final segment, but still a function, we're going to treat it as an Object* getter.
					// in the hopes that it leads to another object that we can resolve the next segment on.  These
					// getter functions must be very simple.

					UObject* CurrentObject = nullptr;
					FProperty* GetterProperty = nullptr;
					FInternalGetterResolver<UObject*> GetterResolver(CurrentObject, GetterProperty);

					FCachedPropertyPath TempPath(Segment);
					if (GetterResolver.Resolve(InContainer, TempPath))
					{
						if (CurrentObject)
						{
							InPropertyPath.SetCanSafelyUsedCachedAddress(false);
							InPropertyPath.SetCachedLastContainer(CurrentObject, SegmentIndex);

							return IteratePropertyPathRecursive(CurrentObject->GetClass(), CurrentObject, SegmentIndex + 1, InPropertyPath, InResolver);
						}
					}
				}
			}
		}

		return false;
	}

	/** Non-UObject helper struct for GetPropertyValueAsString function calls */
	template<typename ContainerType>
	struct FCallGetterFunctionAsStringHelper
	{
		static bool CallGetterFunction(ContainerType* InContainer, UFunction* InFunction, FString& OutValue, uint32 PPF_Flags)
		{
			// Cant call UFunctions on non-UObject containers
			return false;
		}
	};

	/** UObject partial specialization of FCallGetterFunctionHelper */
	template<>
	struct FCallGetterFunctionAsStringHelper<UObject>
	{
		static bool CallGetterFunction(UObject* InContainer, UFunction* InFunction, FString& OutValue, uint32 PPF_Flags)
		{
			// We only support calling functions that return a single value and take no parameters.
			if ( InFunction->NumParms == 1 )
			{
				// Verify there's a return property.
				if ( FProperty* ReturnProperty = InFunction->GetReturnProperty() )
				{
					if ( !InContainer->IsUnreachable() )
					{
						// Create and init a buffer for the function to write to
						TArray<uint8> TempBuffer;
						TempBuffer.AddUninitialized(ReturnProperty->GetElementSize());
						ReturnProperty->InitializeValue(TempBuffer.GetData());

						InContainer->ProcessEvent(InFunction, TempBuffer.GetData());
						ReturnProperty->ExportTextItem_Direct(OutValue, TempBuffer.GetData(), nullptr, nullptr, PPF_Flags);
						return true;
					}
				}
			}

			return false;
		}
	};

	template<typename ContainerType>
	static bool GetPropertyValueAsString(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath, FProperty*& OutProperty, FString& OutValue, uint32 PPF_Flags)
	{
		const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
		int32 ArrayIndex = LastSegment.GetArrayIndex();
		FFieldVariant Field = LastSegment.GetField();

		// We're on the final property in the path, it may be an array property, so check that first.
		if ( FArrayProperty* ArrayProp = Field.Get<FArrayProperty>() )
		{
			// if it's an array property, we need to see if we parsed an index as part of the segment
			// as a user may have baked the index directly into the property path.
			if(ArrayIndex != INDEX_NONE)
			{
				// Property is an array property, so make sure we have a valid index specified
				FScriptArrayHelper_InContainer ArrayHelper(ArrayProp, InContainer);
				if ( ArrayHelper.IsValidIndex(ArrayIndex) )
				{
					OutProperty = ArrayProp->Inner;
					OutProperty->ExportTextItem_Direct(OutValue, static_cast<void*>(ArrayHelper.GetRawPtr(ArrayIndex)), nullptr, nullptr, PPF_Flags);
					return true;
				}
			}
			else
			{
				// No index, so assume we want the array property itself
				if ( !!ArrayProp->ContainerPtrToValuePtr<void>(InContainer) )
				{
					OutProperty = ArrayProp;
					OutProperty->ExportTextItem_InContainer(OutValue, InContainer, nullptr, nullptr, PPF_Flags);
					return true;
				}
			}
		}
		else if(UFunction* Function = Field.Get<UFunction>())
		{
			return FCallGetterFunctionAsStringHelper<ContainerType>::CallGetterFunction(InContainer, Function, OutValue, PPF_Flags);
		}
		else if(FProperty* Property = Field.Get<FProperty>())
		{
			ArrayIndex = ArrayIndex == INDEX_NONE ? 0 : ArrayIndex;
			if( ArrayIndex < Property->ArrayDim )
			{
				if ( void* ValuePtr = Property->ContainerPtrToValuePtr<void>(InContainer, ArrayIndex) )
				{
					OutProperty = Property;
					OutProperty->ExportTextItem_Direct(OutValue, ValuePtr, nullptr, nullptr, PPF_Flags);
					return true;
				}
			}
		}

		return false;
	}

	/** Non-UObject helper struct for SetPropertyValueFromString function calls */
	template<typename ContainerType>
	struct FCallSetterFunctionFromStringHelper
	{
		static bool CallSetterFunction(ContainerType* InContainer, UFunction* InFunction, const FString& InValue, uint32 PPF_Flags)
		{
			// Cant call UFunctions on non-UObject containers
			return false;
		}
	};

	/** UObject partial specialization of FCallSetterFunctionFromStringHelper */
	template<>
	struct FCallSetterFunctionFromStringHelper<UObject>
	{
		static bool CallSetterFunction(UObject* InContainer, UFunction* InFunction, const FString& InValue, uint32 PPF_Flags)
		{
			// We only support calling functions that take a single value and return no parameters
			if ( InFunction->NumParms == 1 && InFunction->GetReturnProperty() == nullptr )
			{
				// Verify there's a single param
				if ( FProperty* ParamProperty = PropertyPathHelpersInternal::GetFirstParamProperty(InFunction) )
				{
					if ( !InContainer->IsUnreachable() )
					{
						// Create and init a buffer for the function to read from
						TArray<uint8> TempBuffer;
						TempBuffer.AddUninitialized(ParamProperty->GetElementSize());
						ParamProperty->InitializeValue(TempBuffer.GetData());

						ParamProperty->ImportText_Direct(*InValue, TempBuffer.GetData(), nullptr, PPF_Flags);
						InContainer->ProcessEvent(InFunction, TempBuffer.GetData());
						return true;
					}
				}
			}

			return false;
		}
	};

	/** Helper: returns the UObject owner of the container, or nullptr when the container is not a UObject. */
	template<typename ContainerType>
	struct FContainerOwnerHelper
	{
		static UObject* GetOwner(ContainerType* /*InContainer*/) { return nullptr; }
	};

	/** UObject specialization: the container is the owner. */
	template<>
	struct FContainerOwnerHelper<UObject>
	{
		static UObject* GetOwner(UObject* InContainer) { return InContainer; }
	};

	template<typename ContainerType>
	static bool SetPropertyValueFromString(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath, const FString& InValue, uint32 PPF_Flags)
	{
		const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
		int32 ArrayIndex = LastSegment.GetArrayIndex();
		FFieldVariant Field = LastSegment.GetField();

		UObject* OwnerObject = FContainerOwnerHelper<ContainerType>::GetOwner(InContainer);

		// We're on the final property in the path, it may be an array property, so check that first.
		if ( FArrayProperty* ArrayProp = Field.Get<FArrayProperty>() )
		{
			// if it's an array property, we need to see if we parsed an index as part of the segment
			// as a user may have baked the index directly into the property path.
			if(ArrayIndex != INDEX_NONE)
			{
				// Property is an array property, so make sure we have a valid index specified
				FScriptArrayHelper_InContainer ArrayHelper(ArrayProp, InContainer);
				if ( ArrayHelper.IsValidIndex(ArrayIndex) )
				{
					ArrayProp->Inner->ImportText_Direct(*InValue, static_cast<void*>(ArrayHelper.GetRawPtr(ArrayIndex)), OwnerObject, PPF_Flags);
					return true;
				}
			}
			else
			{
				// No index, so assume we want the array property itself
				if ( !!ArrayProp->ContainerPtrToValuePtr<void>(InContainer) )
				{
					ArrayProp->ImportText_InContainer(*InValue, InContainer, OwnerObject, PPF_Flags);
					return true;
				}
			}
		}
		else if(UFunction* Function = Field.Get<UFunction>())
		{
			return FCallSetterFunctionFromStringHelper<ContainerType>::CallSetterFunction(InContainer, Function, InValue, PPF_Flags);
		}
		else if(FProperty* Property = Field.Get<FProperty>())
		{
			ArrayIndex = ArrayIndex == INDEX_NONE ? 0 : ArrayIndex;
			if(ArrayIndex < Property->ArrayDim)
			{
				if ( void* ValuePtr = Property->ContainerPtrToValuePtr<void>(InContainer, ArrayIndex) )
				{
					Property->ImportText_Direct(*InValue, ValuePtr, OwnerObject, PPF_Flags);
					return true;
				}
			}
		}

		return false;
	}

	template<typename ContainerType>
	static bool PerformArrayOperation(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath, TFunctionRef<bool(FScriptArrayHelper&,int32)> InOperation)
	{
		const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
		int32 ArrayIndex = LastSegment.GetArrayIndex();

		// We only support array properties right now
		if ( FArrayProperty* ArrayProp = LastSegment.GetField().Get<FArrayProperty>() )
		{
			FScriptArrayHelper_InContainer ArrayHelper(ArrayProp, InContainer);
			return InOperation(ArrayHelper, ArrayIndex);
		}
		return false;
	}

	/** Caches resolve addresses in property paths for later use */
	template<typename ContainerType>
	static bool CacheResolveAddress(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath)
	{
		const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
		int32 ArrayIndex = LastSegment.GetArrayIndex();
		FFieldVariant Field = LastSegment.GetField();

		// We're on the final property in the path, it may be an array property, so check that first.
		if ( FArrayProperty* ArrayProp = Field.Get<FArrayProperty>() )
		{
			// if it's an array property, we need to see if we parsed an index as part of the segment
			// as a user may have baked the index directly into the property path.
			if(ArrayIndex != INDEX_NONE)
			{
				// Property is an array property, so make sure we have a valid index specified
				FScriptArrayHelper_InContainer ArrayHelper(ArrayProp, InContainer);
				if ( ArrayHelper.IsValidIndex(ArrayIndex) )
				{
					if(void* Address = static_cast<void*>(ArrayHelper.GetRawPtr(ArrayIndex)))
					{
						InPropertyPath.ResolveLeaf(Address);
						return true;
					}
				}
			}
			else
			{
				// No index, so assume we want the array property itself
				if(void* Address = ArrayProp->ContainerPtrToValuePtr<void>(InContainer))
				{
					InPropertyPath.ResolveLeaf(Address);
					return true;
				}
			}
		}
		else if(UFunction* Function = Field.Get<UFunction>())
		{
			InPropertyPath.ResolveLeaf(Function);
			return true;
		}
		else if(FProperty* Property = Field.Get<FProperty>())
		{
			ArrayIndex = ArrayIndex == INDEX_NONE ? 0 : ArrayIndex;
			if ( ArrayIndex < Property->ArrayDim )
			{
				if(void* Address = Property->ContainerPtrToValuePtr<void>(InContainer, ArrayIndex))
				{
					InPropertyPath.ResolveLeaf(Address);
					return true;
				}
			}
		}

		return false;
	}

	/** helper function. Copy the values between two resolved paths. It is assumed that CanCopyProperties has been previously called and returned true. */
	static bool CopyResolvedPaths(const FCachedPropertyPath& InDestPropertyPath, const FCachedPropertyPath& InSrcPropertyPath)
	{
		// check we have valid addresses/functions that match
		if(InDestPropertyPath.GetCachedFunction() != nullptr && InSrcPropertyPath.GetCachedFunction() != nullptr)
		{
			// copying via functions is not supported yet
			return false;
		}
		else if(InDestPropertyPath.GetCachedAddress() != nullptr && InSrcPropertyPath.GetCachedAddress() != nullptr)
		{
			const FPropertyPathSegment& DestLastSegment = InDestPropertyPath.GetLastSegment();
			FProperty* DestProperty = CastFieldChecked<FProperty>(DestLastSegment.GetField().ToField());
			FArrayProperty* DestArrayProp = CastField<FArrayProperty>(DestProperty);
			if ( DestArrayProp && DestLastSegment.GetArrayIndex() != INDEX_NONE )
			{
				DestArrayProp->Inner->CopySingleValue(InDestPropertyPath.GetCachedAddress(), InSrcPropertyPath.GetCachedAddress());
			}
			else if(DestProperty->ArrayDim > 1)
			{
				DestProperty->CopyCompleteValue(InDestPropertyPath.GetCachedAddress(), InSrcPropertyPath.GetCachedAddress());
			}
			else if(FBoolProperty* DestBoolProperty = CastField<FBoolProperty>(DestProperty))
			{
				FBoolProperty* SrcBoolProperty = InSrcPropertyPath.GetLastSegment().GetField().Get<FBoolProperty>();
				const bool bValue = SrcBoolProperty->GetPropertyValue(InSrcPropertyPath.GetCachedAddress());
				DestBoolProperty->SetPropertyValue(InDestPropertyPath.GetCachedAddress(), bValue);
			}
			else
			{
				DestProperty->CopySingleValue(InDestPropertyPath.GetCachedAddress(), InSrcPropertyPath.GetCachedAddress());
			}
			return true;
		}

		return false;
	}

	/** Checks if two fully resolved paths can have their values copied between them. Checks the leaf property class to see if they match */
	static bool CanCopyProperties(const FCachedPropertyPath& InDestPropertyPath, const FCachedPropertyPath& InSrcPropertyPath)
	{
		const FPropertyPathSegment& DestLastSegment = InDestPropertyPath.GetLastSegment();
		const FPropertyPathSegment& SrcLastSegment = InSrcPropertyPath.GetLastSegment();

		FProperty* DestProperty = DestLastSegment.GetField().Get<FProperty>();
		FProperty* SrcProperty = SrcLastSegment.GetField().Get<FProperty>();

		if(SrcProperty && DestProperty)
		{
			FArrayProperty* DestArrayProperty = CastField<FArrayProperty>(DestProperty);
			FArrayProperty* SrcArrayProperty = CastField<FArrayProperty>(SrcProperty);

			// If we have a valid index and an array property then we should use the inner property
			FFieldClass* DestClass = (DestArrayProperty != nullptr && DestLastSegment.GetArrayIndex() != INDEX_NONE) ? DestArrayProperty->Inner->GetClass() : DestProperty->GetClass();
			FFieldClass* SrcClass = (SrcArrayProperty != nullptr && SrcLastSegment.GetArrayIndex() != INDEX_NONE) ? SrcArrayProperty->Inner->GetClass() : SrcProperty->GetClass();

			return DestClass == SrcClass && SrcProperty->ArrayDim == DestProperty->ArrayDim;
		}

		return false;
	}

	bool ResolvePropertyPath(UObject* InContainer, const FString& InPropertyPath, FPropertyPathResolver& InResolver)
	{
		if (InContainer)
		{
			FCachedPropertyPath InternalPropertyPath(InPropertyPath);
			return IteratePropertyPathRecursive<UObject>(InContainer->GetClass(), InContainer, 0, InternalPropertyPath, InResolver);
		}

		return false;
	}

	bool ResolvePropertyPath(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, FPropertyPathResolver& InResolver)
	{
		if (InContainer)
		{
			return IteratePropertyPathRecursive<UObject>(InContainer->GetClass(), InContainer, 0, InPropertyPath, InResolver);
		}

		return false;
	}

	bool ResolvePropertyPath(void* InContainer, UStruct* InStruct, const FString& InPropertyPath, FPropertyPathResolver& InResolver)
	{
		FCachedPropertyPath InternalPropertyPath(InPropertyPath);
		return IteratePropertyPathRecursive<void>(InStruct, InContainer, 0, InternalPropertyPath, InResolver);
	}

	bool ResolvePropertyPath(void* InContainer, UStruct* InStruct, const FCachedPropertyPath& InPropertyPath, FPropertyPathResolver& InResolver)
	{
		return IteratePropertyPathRecursive<void>(InStruct, InContainer, 0, InPropertyPath, InResolver);
	}

	FProperty* GetFirstParamProperty(UFunction* InFunction)
	{
		for( TFieldIterator<FProperty> It(InFunction); It && (It->PropertyFlags & CPF_Parm); ++It )
		{
			if( (It->PropertyFlags & CPF_ReturnParm) == 0 )
			{
				return *It;
			}
		}
		return nullptr;
	}

	void CallParentSetters(const FCachedPropertyPath& InPropertyPath)
	{
		int32 LastContainerInPathIndex = InPropertyPath.GetCachedLastContainerInPathIndex();
		int32 IndexAfterCachedLastContainer = InPropertyPath.GetCachedLastContainerInPathIndex() + 1;

		void* ObjectContainerPtr = LastContainerInPathIndex == INDEX_NONE 
			? InPropertyPath.GetCachedContainer()
			: InPropertyPath.GetCachedLastContainerInPath();

		// Compilers will warn if Alloca is used inside a loop, so break Alloca logic out of the loop
		int32 ParentSegmentIndex = INDEX_NONE;

		// Call the topmost setter on the last UObject in path
		const int32 NumSegments = InPropertyPath.GetNumSegments();
		for (int32 Index = IndexAfterCachedLastContainer; Index < NumSegments; Index++)
		{
			const FPropertyPathSegment& ParentSegment = InPropertyPath.GetSegment(Index);
			int32 ParentArrayIndex = ParentSegment.GetArrayIndex();
			FProperty* ParentProperty = CastFieldChecked<FProperty>(ParentSegment.GetField().ToField());
			ParentArrayIndex = ParentArrayIndex == INDEX_NONE ? 0 : ParentArrayIndex;

			if (ParentProperty->HasSetter() && ParentArrayIndex < ParentProperty->ArrayDim)
			{
				ParentSegmentIndex = Index;
				break;
			}
		}

		// Note: We check for HasSetter & ParentArrayIndex valid above
		if (ParentSegmentIndex != INDEX_NONE)
		{
			const FPropertyPathSegment& ParentSegment = InPropertyPath.GetSegment(ParentSegmentIndex);
			int32 ParentArrayIndex = ParentSegment.GetArrayIndex();
			FProperty* ParentProperty = CastFieldChecked<FProperty>(ParentSegment.GetField().ToField());
			ParentArrayIndex = ParentArrayIndex == INDEX_NONE ? 0 : ParentArrayIndex;

			// We want to call the setter with the current value, so just get the pointer to current value via container
			if (void* ParentAddress = ParentProperty->ContainerPtrToValuePtr<void>(ObjectContainerPtr, ParentArrayIndex))
			{
				if (ParentProperty->HasGetter())
				{
					// Call getter if it has one, getter SHOULD be pure and thus won't cause behavioral changes.
					// But this is a read on ParentAddress so we call getter for now

					int32 Size = ParentSegment.GetStruct()->GetPropertiesSize();
					int32 Alignment = ParentSegment.GetStruct()->GetMinAlignment();
					uint8* Temp = (uint8*)FMemory_Alloca_Aligned(Size, Alignment);
					FMemory::Memzero(Temp, Size);

					ParentProperty->InitializeValue(Temp);
					ParentProperty->CallGetter(ObjectContainerPtr, Temp);
					ParentProperty->CallSetter(ObjectContainerPtr, Temp);
					ParentProperty->DestroyValue(Temp);
				}
				else
				{
					ParentProperty->CallSetter(ObjectContainerPtr, ParentAddress);
				}
				return;
			}
		}
	}

	void CallParentGetters(void* OutValue, const FCachedPropertyPath& InPropertyPath, const void* InPropertyAddress)
	{
		int32 LastContainerInPathIndex = InPropertyPath.GetCachedLastContainerInPathIndex();
		int32 IndexAfterCachedLastContainer = InPropertyPath.GetCachedLastContainerInPathIndex() + 1;

		void* ObjectContainerPtr = LastContainerInPathIndex == INDEX_NONE 
			? InPropertyPath.GetCachedContainer()
			: InPropertyPath.GetCachedLastContainerInPath();

		// Helper to get value regardless of property address / if parent getter is called
		auto GetValueFromProperty = [](void* OutValue, const FCachedPropertyPath& InPropertyPath, const void* PropertyAddress)
		{
			FProperty* Property = CastFieldChecked<FProperty>(InPropertyPath.GetLastSegment().GetField().ToField());
			if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
			{
				*static_cast<bool*>(OutValue) = BoolProperty->GetPropertyValue(PropertyAddress);
			}
			else if (ensure(Property))
			{
				Property->CopySingleValue(OutValue, PropertyAddress);
			}
		};

		// Compilers will warn if Alloca is used inside a loop, so break Alloca logic out of the loop
		int32 ParentSegmentIndex = INDEX_NONE;

		// Call the topmost getter on the last UObject in path
		const int32 NumSegments = InPropertyPath.GetNumSegments();
		for (int32 Index = IndexAfterCachedLastContainer; Index < NumSegments; Index++)
		{
			const FPropertyPathSegment& ParentSegment = InPropertyPath.GetSegment(Index);
			int32 ParentArrayIndex = ParentSegment.GetArrayIndex();
			FProperty* ParentProperty = CastFieldChecked<FProperty>(ParentSegment.GetField().ToField());
			ParentArrayIndex = ParentArrayIndex == INDEX_NONE ? 0 : ParentArrayIndex;

			if (ParentProperty->HasGetter() && ParentArrayIndex < ParentProperty->ArrayDim)
			{
				ParentSegmentIndex = Index;
				break;
			}
		}

		// Note: We check for HasGetter & ParentArrayIndex valid above
		if (ParentSegmentIndex != INDEX_NONE)
		{
			const FPropertyPathSegment& ParentSegment = InPropertyPath.GetSegment(ParentSegmentIndex);
			int32 ParentArrayIndex = ParentSegment.GetArrayIndex();
			FProperty* ParentProperty = CastFieldChecked<FProperty>(ParentSegment.GetField().ToField());
			ParentArrayIndex = ParentArrayIndex == INDEX_NONE ? 0 : ParentArrayIndex;

			// We want to call the Getter with the current value, so just get the pointer to current value via container
			void* ParentAddress = ParentProperty->ContainerPtrToValuePtr<void>(ObjectContainerPtr, ParentArrayIndex);
			if (ensure(ParentAddress))
			{
				int32 Size = ParentSegment.GetStruct()->GetPropertiesSize();
				int32 Alignment = ParentSegment.GetStruct()->GetMinAlignment();
				uint8* Temp = (uint8*)FMemory_Alloca_Aligned(Size, Alignment);
				FMemory::Memzero(Temp, Size);

				ParentProperty->InitializeValue(Temp);
				ParentProperty->CallGetter(ObjectContainerPtr, Temp);

				// We resolved the property address earlier & it's containing parent, use this for relative-offset for Temp
				uint8* TempPropertyAddress = Temp + ((uint8*)InPropertyAddress - (uint8*)ParentAddress);
				GetValueFromProperty(OutValue, InPropertyPath, TempPropertyAddress);
				ParentProperty->DestroyValue(Temp);
			}

			return;
		}

		GetValueFromProperty(OutValue, InPropertyPath, InPropertyAddress);
	}
}

FPropertyPathSegment::FPropertyPathSegment()
	: Name(NAME_None)
	, ArrayIndex(INDEX_NONE)
	, Struct(nullptr)
	, Field()
{

}

FPropertyPathSegment::FPropertyPathSegment(int32 InCount, const TCHAR* InString)
	: ArrayIndex(INDEX_NONE)
	, Struct(nullptr)
	, Field()
{
	const TCHAR* PropertyName = nullptr;
	int32 PropertyNameLength = 0;
	PropertyPathHelpers::FindFieldNameAndArrayIndex(InCount, InString, PropertyNameLength, &PropertyName, ArrayIndex);
	ensure(PropertyName != nullptr);
	FString PropertyNameString = FString::ConstructFromPtrSize(PropertyName, PropertyNameLength);
	Name = FName(*PropertyNameString, FNAME_Find);
}

void FPropertyPathSegment::PostSerialize(const FArchive& Ar)
{
	if (Struct)
	{
		// if the struct has been serialized then we're loading a CDO and we need to re-cache the field pointer
		UStruct* ResolveStruct = Struct;
		Struct = nullptr;
		Resolve(ResolveStruct);
	}
}

FPropertyPathSegment FPropertyPathSegment::MakeUnresolvedCopy(const FPropertyPathSegment& ToCopy)
{
	FPropertyPathSegment Segment;
	Segment.Name = ToCopy.Name;
	Segment.ArrayIndex = ToCopy.ArrayIndex;
	return Segment;
}

FFieldVariant FPropertyPathSegment::Resolve(UStruct* InStruct) const
{
	if ( InStruct )
	{
		// only perform the find field work if the structure where this property would resolve to
		// has changed.  If it hasn't changed, the just return the FProperty we found last time.
		if ( InStruct != Struct )
		{
			Struct = InStruct;
			Field = FindUFieldOrFProperty(InStruct, Name);
		}

		return Field;
	}

	return FFieldVariant();
}

FName FPropertyPathSegment::GetName() const
{
	return Name;
}

int32 FPropertyPathSegment::GetArrayIndex() const
{
	return ArrayIndex;
}

FFieldVariant FPropertyPathSegment::GetField() const
{
	return Field;
}

UStruct* FPropertyPathSegment::GetStruct() const
{
	return Struct;
}

FCachedPropertyPath::FCachedPropertyPath()
	: CachedAddress(nullptr)
	, CachedFunction(nullptr)
	, CachedContainer(nullptr)
	, CachedLastContainerInPath(nullptr)
	, CachedLastContainerInPathIndex(INDEX_NONE)
	, bCanSafelyUsedCachedAddress(false)
{
}

FCachedPropertyPath::FCachedPropertyPath(const FString& Path)
	: CachedAddress(nullptr)
	, CachedFunction(nullptr)
	, CachedContainer(nullptr)
	, CachedLastContainerInPath(nullptr)
	, CachedLastContainerInPathIndex(INDEX_NONE)
	, bCanSafelyUsedCachedAddress(false)
{
	MakeFromString(Path);
}

FCachedPropertyPath::FCachedPropertyPath(const TArray<FString>& PathSegments)
	: CachedAddress(nullptr)
	, CachedFunction(nullptr)
	, CachedContainer(nullptr)
	, CachedLastContainerInPath(nullptr)
	, CachedLastContainerInPathIndex(INDEX_NONE)
	, bCanSafelyUsedCachedAddress(false)
{
	for (const FString& Segment : PathSegments)
	{
		Segments.Add(FPropertyPathSegment(Segment.Len(), *Segment));
	}
}

FCachedPropertyPath::FCachedPropertyPath(const FPropertyPathSegment& Segment)
	: CachedAddress(nullptr)
	, CachedFunction(nullptr)
	, CachedContainer(nullptr)
	, CachedLastContainerInPath(nullptr)
	, CachedLastContainerInPathIndex(INDEX_NONE)
	, bCanSafelyUsedCachedAddress(false)
{
	Segments.Add(Segment);
}

FCachedPropertyPath::FCachedPropertyPath(const FPropertyChangedChainEvent& PropertyChangedChainEvent)
	: CachedAddress(nullptr)
	, CachedFunction(nullptr)
	, CachedContainer(nullptr)
	, CachedLastContainerInPath(nullptr)
	, CachedLastContainerInPathIndex(INDEX_NONE)
	, bCanSafelyUsedCachedAddress(false)
{
	MakeFromChangeEvent(PropertyChangedChainEvent);
}

FCachedPropertyPath::FCachedPropertyPath(const FPropertyChangedEvent& PropertyChangedEvent, const FEditPropertyChain& PropertyChain)
	: CachedAddress(nullptr)
	, CachedFunction(nullptr)
	, CachedContainer(nullptr)
	, CachedLastContainerInPath(nullptr)
	, CachedLastContainerInPathIndex(INDEX_NONE)
	, bCanSafelyUsedCachedAddress(false)
{
	MakeFromChangeEvent(PropertyChangedEvent, PropertyChain);
}

FCachedPropertyPath::~FCachedPropertyPath() = default;

void FCachedPropertyPath::MakeFromString(const FString& InPropertyPath)
{
	const TCHAR Delim = TEXT('.');
	const TCHAR* Path = *InPropertyPath;
	int32 Length = InPropertyPath.Len();
	int32 Offset = 0;
	int32 Start = 0;
	while(Offset < Length)
	{
		if (Path[Offset] == Delim)
		{
			Segments.Add(FPropertyPathSegment(Offset - Start, &Path[Start]));
			Start = ++Offset;
		}
		Offset++;
	}
	Segments.Add(FPropertyPathSegment(Length - Start, &Path[Start]));
}

void FCachedPropertyPath::MakeFromChangeEvent(const FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	MakeFromChangeEvent(PropertyChangedChainEvent, PropertyChangedChainEvent.PropertyChain);
}

void FCachedPropertyPath::MakeFromChangeEvent(const FPropertyChangedEvent& PropertyChangedEvent, const FEditPropertyChain& PropertyChain)
{
	for (TDoubleLinkedList<FProperty*>::TIterator It(PropertyChain.GetHead()); It; ++It)
	{
		FProperty* Property = *It;
		FString PropertySegment = Property->GetName();
		const int32 ArrayIndex = PropertyChangedEvent.GetArrayIndex(PropertySegment);
		if (ArrayIndex != INDEX_NONE)
		{
			PropertySegment += FString::Printf(TEXT("[%d]"), ArrayIndex);
		}
		Segments.Add(FPropertyPathSegment(PropertySegment.Len(), *PropertySegment));
	}
}

FCachedPropertyPath FCachedPropertyPath::MakeUnresolvedCopy(const FCachedPropertyPath& ToCopy)
{
	FCachedPropertyPath Path;
	for (const FPropertyPathSegment& Segment : ToCopy.Segments)
	{
		Path.Segments.Add(FPropertyPathSegment::MakeUnresolvedCopy(Segment));
	}
	return Path;
}

FCachedPropertyPath FCachedPropertyPath::MakeParentPath() const
{
	// If there are no more segements, then it's just an empty path.
	if (Segments.Num() == 0)
	{
		return *this;
	}

	FCachedPropertyPath ParentPath = *this;
	ParentPath.Segments.RemoveAt(ParentPath.Segments.Num() - 1);

	return ParentPath;
}

int32 FCachedPropertyPath::GetNumSegments() const
{
	return Segments.Num();
}

const FPropertyPathSegment& FCachedPropertyPath::GetSegment(int32 InSegmentIndex) const
{
	return Segments[InSegmentIndex];
}

const FPropertyPathSegment& FCachedPropertyPath::GetLastSegment() const
{
	return Segments.Last();
}

/** Helper for cache/copy resolver */
struct FInternalCacheResolver : public PropertyPathHelpersInternal::TPropertyPathResolver<FInternalCacheResolver>
{
	template<typename ContainerType>
	bool Resolve_Impl(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath)
	{
		return PropertyPathHelpersInternal::CacheResolveAddress(InContainer, InPropertyPath);
	}
};

bool FCachedPropertyPath::Resolve(UObject* InContainer) const
{
	FInternalCacheResolver Resolver;
	return PropertyPathHelpersInternal::ResolvePropertyPath(InContainer, *this, Resolver);
}

void FCachedPropertyPath::ResolveLeaf(void* InAddress) const
{
	check(CachedFunction == nullptr);
	CachedAddress = InAddress;
}

void FCachedPropertyPath::ResolveLeaf(UFunction* InFunction) const
{
	check(CachedAddress == nullptr);
	CachedFunction = InFunction;
}

void FCachedPropertyPath::SetCanSafelyUsedCachedAddress(bool bInCanSafelyUsedCachedAddress) const
{
	bCanSafelyUsedCachedAddress = bInCanSafelyUsedCachedAddress;
}

void FCachedPropertyPath::SetCachedLastContainer(void* InContainer, int32 InIndex) const
{
	SetCanSafelyUsedCachedAddress(false);
	CachedLastContainerInPath = InContainer;
	CachedLastContainerInPathIndex = InIndex;
}

void* FCachedPropertyPath::GetCachedLastContainerInPath() const
{
	return CachedLastContainerInPath;
}

int32 FCachedPropertyPath::GetCachedLastContainerInPathIndex() const
{
	return CachedLastContainerInPathIndex;
}

bool FCachedPropertyPath::IsResolved() const
{
	return (CachedFunction != nullptr || CachedAddress != nullptr);
}

bool FCachedPropertyPath::IsFullyResolved() const
{
	bool bCachedContainer = CachedContainer != nullptr;
	return bCanSafelyUsedCachedAddress && bCachedContainer && IsResolved();
}

void* FCachedPropertyPath::GetCachedAddress() const
{
	// @TODO: DarenC - Should we? Maybe add a GetCachedAddressUnsafe method.
	// check(bCanSafelyUsedCachedAddress); 
	return CachedAddress;
}

UFunction* FCachedPropertyPath::GetCachedFunction() const
{
	return CachedFunction;
}

FPropertyChangedEvent FCachedPropertyPath::ToPropertyChangedEvent(EPropertyChangeType::Type InChangeType) const
{
	// Path must be resolved
	check(IsResolved());

	// Note: path must not be a to a UFunction
	FPropertyChangedEvent PropertyChangedEvent(CastFieldChecked<FProperty>(GetLastSegment().GetField().ToField()), InChangeType);

	// Set a containing 'struct' if we need to
	if(Segments.Num() > 1)
	{
		PropertyChangedEvent.SetActiveMemberProperty(CastFieldChecked<FProperty>(Segments[Segments.Num() - 2].GetField().ToField()));
	}

	return PropertyChangedEvent;
}

void FCachedPropertyPath::ToEditPropertyChain(FEditPropertyChain& OutPropertyChain) const
{
	// Path must be resolved
	check(IsResolved());

	for (const FPropertyPathSegment& Segment : Segments)
	{
		// Note: path must not be a to a UFunction
		OutPropertyChain.AddTail(CastFieldChecked<FProperty>(Segment.GetField().ToField()));
	}

	OutPropertyChain.SetActivePropertyNode(CastFieldChecked<FProperty>(GetLastSegment().GetField().ToField()));
	if (Segments.Num() > 1)
	{
		OutPropertyChain.SetActiveMemberPropertyNode(CastFieldChecked<FProperty>(Segments[0].GetField().ToField()));
	}
}

FString FCachedPropertyPath::ToString() const
{
	FString OutString;
	for (int32 SegmentIndex = 0; SegmentIndex < Segments.Num(); ++SegmentIndex)
	{
		const FPropertyPathSegment& Segment = Segments[SegmentIndex];

		// Add property name
		OutString += Segment.GetName().ToString();

		// Add array index
		if(Segment.GetArrayIndex() != INDEX_NONE)
		{
			OutString += FString::Printf(TEXT("[%d]"), Segment.GetArrayIndex());
		}

		// Add separator
		if(SegmentIndex < Segments.Num() - 1)
		{
			OutString += TEXT(".");
		}
	}

	return OutString;
}

bool FCachedPropertyPath::operator==(const FString& Other) const
{
	return Equals(Other);
}

bool FCachedPropertyPath::Equals(const FString& Other) const
{
	return ToString() == Other;
}

void* FCachedPropertyPath::GetCachedContainer() const
{
	return CachedContainer;
}

void FCachedPropertyPath::SetCachedContainer(void* InContainer) const
{
	CachedContainer = InContainer;
}

void FCachedPropertyPath::RemoveFromEnd(int32 InNumSegments)
{
	if(InNumSegments <= Segments.Num())
	{
		Segments.RemoveAt(Segments.Num() - 1, InNumSegments);

		// Clear cached data - the path is not the same as the previous Resolve() call
		for (const FPropertyPathSegment& Segment : Segments)
		{
			Segment.Struct = nullptr;
			Segment.Field = FFieldVariant();
		}
		CachedAddress = nullptr;
		CachedFunction = nullptr;
		CachedContainer = nullptr;
		CachedLastContainerInPath = nullptr;
		CachedLastContainerInPathIndex = INDEX_NONE;
		bCanSafelyUsedCachedAddress = false;
	}
}

void FCachedPropertyPath::RemoveFromStart(int32 InNumSegments)
{
	if(InNumSegments <= Segments.Num())
	{
		Segments.RemoveAt(0, InNumSegments);

		// Clear cached data - the path is not the same as the previous Resolve() call
		for (const FPropertyPathSegment& Segment : Segments)
		{
			Segment.Struct = nullptr;
			Segment.Field = FFieldVariant();
		}
		CachedAddress = nullptr;
		CachedFunction = nullptr;
		CachedContainer = nullptr;
		CachedLastContainerInPath = nullptr;
		CachedLastContainerInPathIndex = INDEX_NONE;
		bCanSafelyUsedCachedAddress = false;
	}
}

FProperty* FCachedPropertyPath::GetFProperty() const
{
	return CastField<FProperty>(GetLastSegment().GetField().ToField());
}

namespace PropertyPathHelpers
{
	void FindFieldNameAndArrayIndex(int32 InCount, const TCHAR* InString, int32& OutCount, const TCHAR** OutPropertyName, int32& OutArrayIndex)
	{
		*OutPropertyName = InString;

		// Parse the property name and (optional) array index
		OutArrayIndex = INDEX_NONE;
		OutCount = InCount;
		int32 Offset = 1;
		const TCHAR Bracket = '[';
		while ( Offset < InCount )
		{
			if (InString[Offset] == Bracket)
			{
				OutCount = Offset;
				// here we need to copy - since we need a section of the string only
				FString ArrayIndexString = FString::ConstructFromPtrSize(&InString[Offset + 1], InCount - Offset - 2);
				OutArrayIndex = FCString::Atoi(*ArrayIndexString);
				break;
		}
			Offset++;
		}
	}

	bool GetPropertyValueAsString(UObject* InContainer, const FString& InPropertyPath, FString& OutValue, uint32 PPF_Flags)
	{
		FProperty* Property;
		return GetPropertyValueAsString(InContainer, InPropertyPath, OutValue, Property, PPF_Flags);
	}

	/** Helper for string-based getters */
	struct FInternalStringGetterResolver : public PropertyPathHelpersInternal::TPropertyPathResolver<FInternalStringGetterResolver>
	{
		FInternalStringGetterResolver(FString& InOutValue, FProperty*& InOutProperty, uint32 PPF_Flags)
			: Value(InOutValue)
			, Property(InOutProperty)
			, PPF_Flags(PPF_Flags)
		{
		}

		template<typename ContainerType>
		bool Resolve_Impl(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath)
		{
			return PropertyPathHelpersInternal::GetPropertyValueAsString<ContainerType>(InContainer, InPropertyPath, Property, Value, PPF_Flags);
		}

		FString& Value;
		FProperty*& Property;
		uint32 PPF_Flags;
	};

	bool GetPropertyValueAsString(UObject* InContainer, const FString& InPropertyPath, FString& OutValue, FProperty*& OutProperty, uint32 PPF_Flags)
	{
		check(InContainer);

		FInternalStringGetterResolver Resolver(OutValue, OutProperty, PPF_Flags);
		return ResolvePropertyPath(InContainer, InPropertyPath, Resolver);
	}

	bool GetPropertyValueAsString(void* InContainer, UStruct* InStruct, const FString& InPropertyPath, FString& OutValue, uint32 PPF_Flags)
	{
		FProperty* Property;
		return GetPropertyValueAsString(InContainer, InStruct, InPropertyPath, OutValue, Property, PPF_Flags);
	}

	bool GetPropertyValueAsString(void* InContainer, UStruct* InStruct, const FString& InPropertyPath, FString& OutValue, FProperty*& OutProperty, uint32 PPF_Flags)
	{
		check(InContainer);
		check(InStruct);

		FInternalStringGetterResolver Resolver(OutValue, OutProperty, PPF_Flags);
		return ResolvePropertyPath(InContainer, InStruct, InPropertyPath, Resolver);
	}

	bool GetPropertyValueAsString(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, FString& OutValue, uint32 PPF_Flags)
	{
		check(InContainer);

		FProperty* Property;
		FInternalStringGetterResolver Resolver(OutValue, Property, PPF_Flags);
		return ResolvePropertyPath(InContainer, InPropertyPath, Resolver);
	}

	bool GetPropertyValueAsString(void* InContainer, UStruct* InStruct, const FCachedPropertyPath& InPropertyPath, FString& OutValue, uint32 PPF_Flags)
	{
		check(InContainer);
		check(InStruct);

		FProperty* Property;
		FInternalStringGetterResolver Resolver(OutValue, Property, PPF_Flags);
		return ResolvePropertyPath(InContainer, InStruct, InPropertyPath, Resolver);
	}

	/** Helper for string-based setters */
	struct FInternalStringSetterResolver : public PropertyPathHelpersInternal::TPropertyPathResolver<FInternalStringSetterResolver>
	{
		FInternalStringSetterResolver(const FString& InValueAsString, uint32 PPF_Flags)
			: Value(InValueAsString)
			, PPF_Flags(PPF_Flags)
		{
		}

		template<typename ContainerType>
		bool Resolve_Impl(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath)
		{
			return PropertyPathHelpersInternal::SetPropertyValueFromString<ContainerType>(InContainer, InPropertyPath, Value, PPF_Flags);
		}
	
		const FString& Value;
		uint32 PPF_Flags;
	};

	bool SetPropertyValueFromString(UObject* InContainer, const FString& InPropertyPath, const FString& InValue, uint32 PPF_Flags)
	{
		check(InContainer);

		FInternalStringSetterResolver Resolver(InValue, PPF_Flags);
		return ResolvePropertyPath(InContainer, InPropertyPath, Resolver);
	}

	bool SetPropertyValueFromString(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, const FString& InValue, uint32 PPF_Flags)
	{
		check(InContainer);

		FInternalStringSetterResolver Resolver(InValue, PPF_Flags);
		return ResolvePropertyPath(InContainer, InPropertyPath, Resolver);
	}

	bool SetPropertyValueFromString(void* InContainer, UStruct* InStruct, const FString& InPropertyPath, const FString& InValue, uint32 PPF_Flags)
	{
		check(InContainer);
		check(InStruct);

		FInternalStringSetterResolver Resolver(InValue, PPF_Flags);
		return ResolvePropertyPath(InContainer, InStruct, InPropertyPath, Resolver);
	}

	bool SetPropertyValueFromString(void* InContainer, UStruct* InStruct, const FCachedPropertyPath& InPropertyPath, const FString& InValue, uint32 PPF_Flags)
	{
		check(InContainer);
		check(InStruct);

		FInternalStringSetterResolver Resolver(InValue, PPF_Flags);
		return ResolvePropertyPath(InContainer, InStruct, InPropertyPath, Resolver);
	}

	bool SetPropertyValue(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, const UScriptStruct* InScriptStruct, const uint8* InValue)
	{
		PropertyPathHelpersInternal::FPropertyStructView StructView(InScriptStruct, InValue);
		return SetPropertyValue(InContainer, InPropertyPath, StructView);
	}

	bool SetPropertyValue(UObject* InContainer, const FString& InPropertyPath, const UScriptStruct* InScriptStruct, const uint8* InValue)
	{
		PropertyPathHelpersInternal::FPropertyStructView StructView(InScriptStruct, InValue);
		return SetPropertyValue(InContainer, InPropertyPath, StructView);
	}

	bool CopyPropertyValue(UObject* InContainer, const FCachedPropertyPath& InDestPropertyPath, const FCachedPropertyPath& InSrcPropertyPath)
	{
		if(InDestPropertyPath.IsFullyResolved() && InSrcPropertyPath.IsFullyResolved())
		{
			return PropertyPathHelpersInternal::CopyResolvedPaths(InDestPropertyPath, InSrcPropertyPath);
		}
		else
		{
			FInternalCacheResolver DestResolver;
			FInternalCacheResolver SrcResolver;
			if(ResolvePropertyPath(InContainer, InDestPropertyPath, DestResolver) && ResolvePropertyPath(InContainer, InSrcPropertyPath, SrcResolver))
			{
				if(InDestPropertyPath.IsResolved() && InSrcPropertyPath.IsResolved())
				{
					if(PropertyPathHelpersInternal::CanCopyProperties(InDestPropertyPath, InSrcPropertyPath))
					{
						return PropertyPathHelpersInternal::CopyResolvedPaths(InDestPropertyPath, InSrcPropertyPath);
					}
				}
			}
		}

		return false;
	}

	bool CopyPropertyValueFast(UObject* InContainer, const FCachedPropertyPath& InDestPropertyPath, const FCachedPropertyPath& InSrcPropertyPath)
	{
		check(InContainer == InDestPropertyPath.GetCachedContainer());
		check(InContainer == InSrcPropertyPath.GetCachedContainer());
		checkSlow(InDestPropertyPath.IsResolved());
		checkSlow(InSrcPropertyPath.IsResolved());
		checkSlow(PropertyPathHelpersInternal::CanCopyProperties(InDestPropertyPath, InSrcPropertyPath));

		return PropertyPathHelpersInternal::CopyResolvedPaths(InDestPropertyPath, InSrcPropertyPath);
	}

	/** Helper for array operations*/
	struct FInternalArrayOperationResolver : public PropertyPathHelpersInternal::TPropertyPathResolver<FInternalArrayOperationResolver>
	{
		FInternalArrayOperationResolver(TFunctionRef<bool(FScriptArrayHelper&,int32)> InOperation)
			: Operation(InOperation)
		{
		}

		template<typename ContainerType>
		bool Resolve_Impl(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath)
		{
			return PropertyPathHelpersInternal::PerformArrayOperation<ContainerType>(InContainer, InPropertyPath, Operation);
		}
	
		TFunctionRef<bool(FScriptArrayHelper&,int32)> Operation;
	};

	bool PerformArrayOperation(UObject* InContainer, const FString& InPropertyPath, TFunctionRef<bool(FScriptArrayHelper&,int32)> InOperation)
	{
		FInternalArrayOperationResolver Resolver(InOperation);
		return ResolvePropertyPath(InContainer, InPropertyPath, Resolver);
	}

	bool PerformArrayOperation(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, TFunctionRef<bool(FScriptArrayHelper&,int32)> InOperation)
	{
		FInternalArrayOperationResolver Resolver(InOperation);
		return ResolvePropertyPath(InContainer, InPropertyPath, Resolver);
	}
}