// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyPathHelpersTest.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyPathHelpersTest)

#if WITH_DEV_AUTOMATION_TESTS

DEFINE_LOG_CATEGORY_STATIC(LogPropertyPathHelpersTest, Log, All);

/**
 * Coverage:
 * 
 * SetPrimitives (Bool, Integer, Float)
 * SetInvalidInputIsNoOp
 * SetString
 * SetEnum
 * SetStruct
 * SetInnerObject
 * SetStructData
 * SetStructMember
 * SetStructInnerObjectMember
 * SetInnerStructMember
 * SetPrimitivesInnerObject
 * SetStringInnerObject
 * SetStructInnerObject
 * SetStructMemberInnerObject
 * SetInnerStructMemberInnerObject
 * SetInnerStructMemberEnumInnerObject
 * 
 * GetPrimitives
 * GetString
 * GetStruct
 * GetInnerObject
 * GetEnum
 * GetStructMember
 * GetStructInnerObjectMember
 * GetInnerStructMember
 * GetPrimitivesInnerObject
 * GetStringInnerObject
 * GetStructInnerObject
 * GetStructMemberInnerObject
 * GetInnerStructMemberInnerObject
 * GetInnerStructMemberEnumInnerObject
 * 
 * Note: 
 * When a struct setter is called due to a struct member set we expect getter to be called. 
 * This is because to call the struct setter we had to read the current struct value.
 */

// Begin Tests

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetPrimitivesTest, "System.PropertyPath.SetPrimitivesTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetPrimitivesTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->Bool = true;

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("Bool"), true);
		GPassing &= TestEqual(TEXT("Primitive Bool Set"), Test.Object->Bool, true);
		GPassing &= TestEqual(TEXT("Primitive Bool Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Primitive Bool Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Bool Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->Integer = 1;

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("Integer"), 1);
		GPassing &= TestEqual(TEXT("Primitive Integer Set"), Test.Object->Integer, 1);
		GPassing &= TestEqual(TEXT("Primitive Integer Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Primitive Integer Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Integer Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->SetFloat(1.5f);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("Float"), 1.5f);
		GPassing &= TestEqual(TEXT("Primitive Float Set"), Test.Object->Float, 1.5f);
		GPassing &= TestEqual(TEXT("Primitive Float Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Primitive Float Setter Called"), Test.Object->IsSetterCalled(), true);
		GPassing &= TestEqual(TEXT("Primitive Float Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetInvalidInputIsNoOp, "System.PropertyPath.SetInvalidInputIsNoOp", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetInvalidInputIsNoOp::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	// Note: Skipping CppTests here since these are No Ops.

	{
		FPropertyPathTestBed Test = {};

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("Bool"), Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Bool Invalid Input Set No Op"), Test.Object->Bool, false);
		GPassing &= TestEqual(TEXT("Bool Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Bool Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("Integer"), Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Integer Invalid Input Set No Op"), Test.Object->Integer, 0);
		GPassing &= TestEqual(TEXT("Integer Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Integer Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("Float"), Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Float Invalid Input Set No Op"), Test.Object->Float, 0.0f);
		GPassing &= TestEqual(TEXT("Float Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Float Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("Struct"), 1);
		GPassing &= TestEqual(TEXT("Struct Invalid Input Set No Op"), Test.Object->Struct, Test.DefaultStruct);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetStringTest, "System.PropertyPath.SetStringTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetStringTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->String = FString("NewValue");

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("String"), FString("NewValue"));
		GPassing &= TestEqual(TEXT("String Set"), Test.Object->String, FString("NewValue"));
		GPassing &= TestEqual(TEXT("String Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("String Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("String Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetEnumTest, "System.PropertyPath.SetEnumTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetEnumTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->EnumTwo = Two;

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("EnumTwo"), Two);
		GPassing &= TestEqual(TEXT("Enum Set"), Test.Object->EnumTwo, Two);
		GPassing &= TestEqual(TEXT("Enum Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Enum Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Enum Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	{
		// Test for
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->EnumOne = Four;
		CppTest.Object->EnumTwo = Three;
		CppTest.Object->EnumThree = Two;
		CppTest.Object->EnumFour = One;

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("EnumOne"), Four);
		CppTest.Object->EnumOne = Four;
		GPassing &= TestEqual(TEXT("Enum Set Cpp Equivalent"), *Test.Object, *CppTest.Object);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("EnumTwo"), Three);
		CppTest.Object->EnumTwo = Three;
		GPassing &= TestEqual(TEXT("Enum Set Cpp Equivalent"), *Test.Object, *CppTest.Object);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("EnumThree"), Two);
		CppTest.Object->EnumThree = Two;
		GPassing &= TestEqual(TEXT("Enum Set Cpp Equivalent"), *Test.Object, *CppTest.Object);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("EnumFour"), One);
		CppTest.Object->EnumFour = One;
		GPassing &= TestEqual(TEXT("Enum Set Cpp Equivalent"), *Test.Object, *CppTest.Object);

		GPassing &= TestEqual(TEXT("Enum Close No Overwrite Set"), Test.Object->EnumOne, Four);
		GPassing &= TestEqual(TEXT("Enum Close No Overwrite Set"), Test.Object->EnumTwo, Three);
		GPassing &= TestEqual(TEXT("Enum Close No Overwrite Set"), Test.Object->EnumThree, Two);
		GPassing &= TestEqual(TEXT("Enum Close No Overwrite Set"), Test.Object->EnumFour, One);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("EnumFour"), Four);
		CppTest.Object->EnumFour = Four;
		GPassing &= TestEqual(TEXT("Enum Set Cpp Equivalent"), *Test.Object, *CppTest.Object);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("EnumThree"), Three);
		CppTest.Object->EnumThree = Three;
		GPassing &= TestEqual(TEXT("Enum Set Cpp Equivalent"), *Test.Object, *CppTest.Object);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("EnumTwo"), Two);
		CppTest.Object->EnumTwo = Two;
		GPassing &= TestEqual(TEXT("Enum Set Cpp Equivalent"), *Test.Object, *CppTest.Object);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("EnumOne"), One);
		CppTest.Object->EnumOne = One;
		GPassing &= TestEqual(TEXT("Enum Set Cpp Equivalent"), *Test.Object, *CppTest.Object);

		GPassing &= TestEqual(TEXT("Enum Close No Overwrite Set"), Test.Object->EnumOne, One);
		GPassing &= TestEqual(TEXT("Enum Close No Overwrite Set"), Test.Object->EnumTwo, Two);
		GPassing &= TestEqual(TEXT("Enum Close No Overwrite Set"), Test.Object->EnumThree, Three);
		GPassing &= TestEqual(TEXT("Enum Close No Overwrite Set"), Test.Object->EnumFour, Four);

		GPassing &= TestEqual(TEXT("Enum Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Enum Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetStructTest, "System.PropertyPath.SetStructTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetStructTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->SetStruct(Test.ModifiedStruct);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("Struct"), Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Set"), Test.Object->Struct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), true);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->SetStructRef(Test.ModifiedStruct);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("StructRef"), Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Ref Set"), Test.Object->StructRef, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Ref Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), true);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->SetStructConstRef(Test.ModifiedStruct);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("StructConstRef"), Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Const Ref Set"), Test.Object->StructConstRef, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Const Ref Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), true);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetInnerObjectTest, "System.PropertyPath.SetInnerObjectTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetInnerObjectTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->InnerObject = CppTest.ModifiedObject;

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("InnerObject"), Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Inner Object Set"), Test.Object->InnerObject, Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Inner Object Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Inner Object Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Inner Object Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetStructDataTest, "System.PropertyPath.SetStructDataTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetStructDataTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->SetStruct(Test.ModifiedStruct);
		
		UScriptStruct* ScriptStruct = TBaseStructure<FPropertyPathTestStruct>::Get();
		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("Struct"), ScriptStruct, (const uint8*)&Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Data Set"), Test.Object->Struct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Data Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), true);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->SetStructRef(Test.ModifiedStruct);

		UScriptStruct* ScriptStruct = TBaseStructure<FPropertyPathTestStruct>::Get();
		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("StructRef"), ScriptStruct, (const uint8*)&Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Ref Data Set"), Test.Object->StructRef, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Ref Data Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), true);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->SetStructConstRef(Test.ModifiedStruct);

		UScriptStruct* ScriptStruct = TBaseStructure<FPropertyPathTestStruct>::Get();
		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("StructConstRef"), ScriptStruct, (const uint8*)&Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Const Ref Data Set"), Test.Object->StructConstRef, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Const Ref Data Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), true);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetStructMemberTest, "System.PropertyPath.SetStructMemberTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetStructMemberTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->Struct.Float = 1.5f;
		CppTest.Object->SetStruct(CppTest.Object->Struct);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("Struct.Float"), 1.5f);
		GPassing &= TestEqual(TEXT("Struct Member Set"), Test.Object->Struct.Float, 1.5f);
		GPassing &= TestEqual(TEXT("Struct Member Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), true);

		// Emphasis, see note under coverage at top
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);

		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->StructRef.Float = 1.5f;
		CppTest.Object->SetStructRef(CppTest.Object->StructRef);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("StructRef.Float"), 1.5f);
		GPassing &= TestEqual(TEXT("Struct Ref Member Set"), Test.Object->StructRef.Float, 1.5f);
		GPassing &= TestEqual(TEXT("Struct Ref Member Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), true);

		// Emphasis, see note under coverage at top
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);

		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->StructConstRef.Float = 1.5f;
		CppTest.Object->SetStructConstRef(CppTest.Object->StructConstRef);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("StructConstRef.Float"), 1.5f);
		GPassing &= TestEqual(TEXT("Struct Const Ref Member Set"), Test.Object->StructConstRef.Float, 1.5f);
		GPassing &= TestEqual(TEXT("Struct Const Ref Member Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), true);

		// Emphasis, see note under coverage at top
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);

		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetStructInnerObjectMemberTest, "System.PropertyPath.SetStructInnerObjectMemberTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetStructInnerObjectMemberTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->Struct.InnerObject = CppTest.ModifiedObject;
		CppTest.Object->SetStruct(CppTest.Object->Struct);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("Struct.InnerObject"), Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Struct Inner Object Member Set"), Test.Object->Struct.InnerObject, Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Struct Inner Object Member Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), true);

		// Emphasis, see note under coverage at top
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);

		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->StructRef.InnerObject = CppTest.ModifiedObject;
		CppTest.Object->SetStructRef(CppTest.Object->StructRef);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("StructRef.InnerObject"), Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Struct Ref Inner Object Member Set"), Test.Object->StructRef.InnerObject, Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Struct Ref Inner Object Member Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), true);

		// Emphasis, see note under coverage at top
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);

		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->StructConstRef.InnerObject = CppTest.ModifiedObject;
		CppTest.Object->SetStructConstRef(CppTest.Object->StructConstRef);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("StructConstRef.InnerObject"), Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Struct Const Ref Inner Object Member Set"), Test.Object->StructConstRef.InnerObject, Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Struct Const Ref Inner Object Member Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), true);

		// Emphasis, see note under coverage at top
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);

		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetInnerStructMemberTest, "System.PropertyPath.SetInnerStructMemberTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetInnerStructMemberTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->Struct.InnerStruct.Float = 1.5f;
		CppTest.Object->SetStruct(CppTest.Object->Struct);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("Struct.InnerStruct.Float"), 1.5f);
		GPassing &= TestEqual(TEXT("Inner Struct Member Set"), Test.Object->Struct.InnerStruct.Float, 1.5f);
		GPassing &= TestEqual(TEXT("Inner Struct Member Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), true);

		// Emphasis, see note under coverage at top
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);

		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetPrimitivesInnerObjectTest, "System.PropertyPath.SetPrimitivesInnerObjectTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetPrimitivesInnerObjectTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->InnerObject->Bool = true;

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("InnerObject.Bool"), true);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Bool Set"), Test.Object->InnerObject->Bool, true);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Bool Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Bool Setter Called"), Test.Object->InnerObject->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Bool Getter Called"), Test.Object->InnerObject->IsGetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Bool Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Bool Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->InnerObject->ResetGetterSetterFlags();
		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->InnerObject->Integer = 1;

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("InnerObject.Integer"), 1);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Integer Set"), Test.Object->InnerObject->Integer, 1);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Integer Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Integer Setter Called"), Test.Object->InnerObject->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Integer Getter Called"), Test.Object->InnerObject->IsGetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Bool Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Bool Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->InnerObject->ResetGetterSetterFlags();
		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->InnerObject->SetFloat(1.5f);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("InnerObject.Float"), 1.5f);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Float Set"), Test.Object->InnerObject->Float, 1.5f);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Float Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Float Setter Called"), Test.Object->InnerObject->IsSetterCalled(), true);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Float Getter Called"), Test.Object->InnerObject->IsGetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Float Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Float Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->InnerObject->ResetGetterSetterFlags();
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetStringInnerObjectTest, "System.PropertyPath.SetStringInnerObjectTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetStringInnerObjectTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->InnerObject->String = FString("NewValue");

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("InnerObject.String"), FString("NewValue"));
		GPassing &= TestEqual(TEXT("Inner Object String Set"), Test.Object->InnerObject->String, FString("NewValue"));
		GPassing &= TestEqual(TEXT("Inner Object String Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Inner Object String Setter Called"), Test.Object->InnerObject->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Inner Object String Getter Called"), Test.Object->InnerObject->IsGetterCalled(), false);
		GPassing &= TestEqual(TEXT("String Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("String Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->InnerObject->ResetGetterSetterFlags();
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetStructInnerObjectTest, "System.PropertyPath.SetStructInnerObjectTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetStructInnerObjectTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->InnerObject->SetStruct(Test.ModifiedStruct);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("InnerObject.Struct"), Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Inner Object Struct Set"), Test.Object->InnerObject->Struct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Inner Object Struct Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Inner Object Struct Setter Called"), Test.Object->InnerObject->IsSetterCalled(), true);
		GPassing &= TestEqual(TEXT("Inner Object Struct Getter Called"), Test.Object->InnerObject->IsGetterCalled(), false);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->InnerObject->ResetGetterSetterFlags();
		Test.Object->ResetGetterSetterFlags();
	}
	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetStructMemberInnerObjectTest, "System.PropertyPath.SetStructMemberInnerObjectTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetStructMemberInnerObjectTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->InnerObject->Struct.Float = 1.5f;
		CppTest.Object->InnerObject->SetStruct(CppTest.Object->InnerObject->Struct);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("InnerObject.Struct.Float"), 1.5f);
		GPassing &= TestEqual(TEXT("Inner Object Struct Member Set"), Test.Object->InnerObject->Struct.Float, 1.5f);
		GPassing &= TestEqual(TEXT("Inner Object Struct Member Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Inner Object Struct Setter Called"), Test.Object->InnerObject->IsSetterCalled(), true);

		// Emphasis, see note under coverage at top
		GPassing &= TestEqual(TEXT("Inner Object Struct Getter Called"), Test.Object->InnerObject->IsGetterCalled(), true);

		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetInnerStructMemberInnerObjectTest, "System.PropertyPath.SetInnerStructMemberInnerObjectTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetInnerStructMemberInnerObjectTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->InnerObject->Struct.InnerStruct.Float = 1.5f;
		CppTest.Object->InnerObject->SetStruct(CppTest.Object->InnerObject->Struct);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("InnerObject.Struct.InnerStruct.Float"), 1.5f);
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Member Set"), Test.Object->InnerObject->Struct.InnerStruct.Float, 1.5f);
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Member Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Setter Called"), Test.Object->InnerObject->IsSetterCalled(), true);

		// Emphasis, see note under coverage at top
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Getter Called"), Test.Object->InnerObject->IsGetterCalled(), true);

		GPassing &= TestEqual(TEXT("Inner Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Inner Struct Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSetInnerStructMemberEnumInnerObjectTest, "System.PropertyPath.SetInnerStructMemberEnumInnerObjectTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FSetInnerStructMemberEnumInnerObjectTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.Object->InnerObject->Struct.InnerStruct.EnumTwo = Two;
		CppTest.Object->InnerObject->SetStruct(CppTest.Object->InnerObject->Struct);

		PropertyPathHelpers::SetPropertyValue(Test.Object, FString("InnerObject.Struct.InnerStruct.EnumTwo"), Two);
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Member Enum Set"), Test.Object->InnerObject->Struct.InnerStruct.EnumTwo, Two);
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Member Enum Set Cpp Equivalent"), *Test.Object, *CppTest.Object);
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Setter Called"), Test.Object->InnerObject->IsSetterCalled(), true);

		// Emphasis, see note under coverage at top
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Getter Called"), Test.Object->InnerObject->IsGetterCalled(), true);

		GPassing &= TestEqual(TEXT("Inner Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Inner Struct Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

 IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGetPrimitivesTest, "System.PropertyPath.GetPrimitivesTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
 bool FGetPrimitivesTest::RunTest(const FString& Parameters)
 {
	 bool GPassing = true;

	 {
	 	FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.Bool = CppTest.Object->Bool;

	 	PropertyPathHelpers::GetPropertyValue(Test.Object, FString("Bool"), Test.ModifiedStruct.Bool);
	 	GPassing &= TestEqual(TEXT("Primitive Bool Get"), Test.DefaultStruct.Bool, Test.ModifiedStruct.Bool);
		GPassing &= TestEqual(TEXT("Primitive Bool Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
	 	GPassing &= TestEqual(TEXT("Primitive Bool Setter Called"), Test.Object->IsSetterCalled(), false);
	 	GPassing &= TestEqual(TEXT("Primitive Bool Getter Called"), Test.Object->IsGetterCalled(), false);
	 	Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.Integer = CppTest.Object->Integer;

	 	PropertyPathHelpers::GetPropertyValue(Test.Object, FString("Integer"), Test.ModifiedStruct.Integer);
	 	GPassing &= TestEqual(TEXT("Primitive Integer Get"), Test.DefaultStruct.Integer, Test.ModifiedStruct.Integer);
		GPassing &= TestEqual(TEXT("Primitive Integer Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
	 	GPassing &= TestEqual(TEXT("Primitive Integer Setter Called"), Test.Object->IsSetterCalled(), false);
	 	GPassing &= TestEqual(TEXT("Primitive Integer Getter Called"), Test.Object->IsGetterCalled(), false);
	 	Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.Float = CppTest.Object->GetFloat();

	 	PropertyPathHelpers::GetPropertyValue(Test.Object, FString("Float"), Test.ModifiedStruct.Float);
	 	GPassing &= TestEqual(TEXT("Primitive Float Get"), Test.DefaultStruct.Float, Test.ModifiedStruct.Float);
		GPassing &= TestEqual(TEXT("Primitive Float Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
	 	GPassing &= TestEqual(TEXT("Primitive Float Setter Called"), Test.Object->IsSetterCalled(), false);
	 	GPassing &= TestEqual(TEXT("Primitive Float Getter Called"), Test.Object->IsGetterCalled(), true);
	 	Test.Object->ResetGetterSetterFlags();
	}

	 return GPassing;
 }


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGetStringTest, "System.PropertyPath.GetStringTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FGetStringTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.String = CppTest.Object->String;

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("String"), Test.ModifiedStruct.String);
		GPassing &= TestEqual(TEXT("String Get"), Test.DefaultStruct.String, Test.ModifiedStruct.String);
		GPassing &= TestEqual(TEXT("String Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("String Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("String Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGetEnumTest, "System.PropertyPath.GetEnumTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FGetEnumTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.EnumTwo = CppTest.Object->EnumTwo;

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("EnumTwo"), Test.ModifiedStruct.EnumTwo);
		GPassing &= TestEqual(TEXT("Enum Get"), Test.DefaultStruct.EnumTwo, Test.ModifiedStruct.EnumTwo);
		GPassing &= TestEqual(TEXT("Enum Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Enum Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Enum Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGetStructTest, "System.PropertyPath.GetStructTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FGetStructTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct = CppTest.Object->GetStruct();

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("Struct"), Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Get"), Test.DefaultStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);
		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct = CppTest.Object->GetStructRef();

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("StructRef"), Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Ref Get"), Test.DefaultStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Ref Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);
		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct = CppTest.Object->GetStructConstRef();

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("StructConstRef"), Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Const Ref Get"), Test.DefaultStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Const Ref Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGetInnerObjectTest, "System.PropertyPath.GetInnerObjectTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FGetInnerObjectTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedObject = CppTest.Object->InnerObject;

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("InnerObject"), Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Inner Object Get"), Test.Object->InnerObject, Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Inner Object Get Cpp Equivalent"), *CppTest.ModifiedObject, *Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Inner Object Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Inner Object Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGetStructMemberTest, "System.PropertyPath.GetStructMemberTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FGetStructMemberTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.Float = CppTest.Object->GetStruct().Float;

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("Struct.Float"), Test.ModifiedStruct.Float);
		GPassing &= TestEqual(TEXT("Struct Member Get"), Test.DefaultStruct.Float, Test.ModifiedStruct.Float);
		GPassing &= TestEqual(TEXT("Struct Member Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);

		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.Float = CppTest.Object->GetStructRef().Float;

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("StructRef.Float"), Test.ModifiedStruct.Float);
		GPassing &= TestEqual(TEXT("Struct Ref Member Get"), Test.DefaultStruct.Float, Test.ModifiedStruct.Float);
		GPassing &= TestEqual(TEXT("Struct Ref Member Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);

		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.Float = CppTest.Object->GetStructConstRef().Float;

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("StructConstRef.Float"), Test.ModifiedStruct.Float);
		GPassing &= TestEqual(TEXT("Struct Const Ref Member Get"), Test.DefaultStruct.Float, Test.ModifiedStruct.Float);
		GPassing &= TestEqual(TEXT("Struct Const Ref Member Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);

		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGetStructInnerObjectMemberTest, "System.PropertyPath.GetStructInnerObjectMemberTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FGetStructInnerObjectMemberTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedObject = CppTest.Object->GetStruct().InnerObject;

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("Struct.InnerObject"), Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Struct Member Get"), *Test.ModifiedObject, *Test.Object->Struct.InnerObject);
		GPassing &= TestEqual(TEXT("Struct Member Get Cpp Equivalent"), *CppTest.ModifiedObject, *Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);

		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedObject = CppTest.Object->GetStructRef().InnerObject;

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("StructRef.InnerObject"), Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Struct Ref Member Get"), *Test.ModifiedObject, *Test.Object->StructRef.InnerObject);
		GPassing &= TestEqual(TEXT("Struct Ref Member Get Cpp Equivalent"), *CppTest.ModifiedObject, *Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);

		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedObject = CppTest.Object->GetStructConstRef().InnerObject;

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("StructConstRef.InnerObject"), Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Struct Const Ref Member Get"), *Test.ModifiedObject, *Test.Object->StructConstRef.InnerObject);
		GPassing &= TestEqual(TEXT("Struct Const Ref Member Get Cpp Equivalent"), *CppTest.ModifiedObject, *Test.ModifiedObject);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), true);

		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGetInnerStructMemberTest, "System.PropertyPath.GetInnerStructMemberTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FGetInnerStructMemberTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.Float = CppTest.Object->GetStruct().InnerStruct.Float;

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("Struct.InnerStruct.Float"), Test.ModifiedStruct.Float);
		GPassing &= TestEqual(TEXT("Inner Struct Member Get"), Test.DefaultStruct.InnerStruct.Float, Test.ModifiedStruct.Float);
		GPassing &= TestEqual(TEXT("Inner Struct Member Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Inner Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Inner Struct Getter Called"), Test.Object->IsGetterCalled(), true);

		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGetPrimitivesInnerObjectTest, "System.PropertyPath.GetPrimitivesInnerObjectTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FGetPrimitivesInnerObjectTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.Bool = CppTest.Object->InnerObject->Bool;

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("InnerObject.Bool"), Test.ModifiedStruct.Bool);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Bool Get"), Test.DefaultStruct.Bool, Test.ModifiedStruct.Bool);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Bool Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Bool Setter Called"), Test.Object->InnerObject->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Bool Getter Called"), Test.Object->InnerObject->IsGetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Bool Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Bool Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->InnerObject->ResetGetterSetterFlags();
		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.Integer = CppTest.Object->InnerObject->Integer;

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("InnerObject.Integer"), Test.ModifiedStruct.Integer);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Integer Get"), Test.DefaultStruct.Integer, Test.ModifiedStruct.Integer);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Integer Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Integer Setter Called"), Test.Object->InnerObject->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Integer Getter Called"), Test.Object->InnerObject->IsGetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Bool Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Bool Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->InnerObject->ResetGetterSetterFlags();
		Test.Object->ResetGetterSetterFlags();
	}

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.Float = CppTest.Object->InnerObject->GetFloat();

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("InnerObject.Float"), Test.ModifiedStruct.Float);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Float Get"), Test.DefaultStruct.Float, Test.ModifiedStruct.Float);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Float Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Float Setter Called"), Test.Object->InnerObject->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Inner Object Float Getter Called"), Test.Object->InnerObject->IsGetterCalled(), true);
		GPassing &= TestEqual(TEXT("Primitive Float Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Primitive Float Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->InnerObject->ResetGetterSetterFlags();
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGetStringInnerObjectTest, "System.PropertyPath.GetStringInnerObjectTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FGetStringInnerObjectTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.String = CppTest.Object->InnerObject->String;

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("InnerObject.String"), Test.ModifiedStruct.String);
		GPassing &= TestEqual(TEXT("Inner Object String Get"), Test.DefaultStruct.String, Test.ModifiedStruct.String);
		GPassing &= TestEqual(TEXT("Inner Object String Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Inner Object String Setter Called"), Test.Object->InnerObject->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Inner Object String Getter Called"), Test.Object->InnerObject->IsGetterCalled(), false);
		GPassing &= TestEqual(TEXT("String Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("String Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->InnerObject->ResetGetterSetterFlags();
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGetStructInnerObjectTest, "System.PropertyPath.GetStructInnerObjectTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FGetStructInnerObjectTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct = CppTest.Object->InnerObject->GetStruct();

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("InnerObject.Struct"), Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Inner Object Struct Get"), Test.DefaultStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Inner Object Struct Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Inner Object Struct Setter Called"), Test.Object->InnerObject->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Inner Object Struct Getter Called"), Test.Object->InnerObject->IsGetterCalled(), true);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->InnerObject->ResetGetterSetterFlags();
		Test.Object->ResetGetterSetterFlags();
	}
	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGetStructMemberInnerObjectTest, "System.PropertyPath.GetStructMemberInnerObjectTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FGetStructMemberInnerObjectTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.Float = CppTest.Object->InnerObject->GetStruct().Float;

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("InnerObject.Struct.Float"), Test.ModifiedStruct.Float);
		GPassing &= TestEqual(TEXT("Inner Object Struct Member Get"), Test.DefaultStruct.Float, Test.ModifiedStruct.Float);
		GPassing &= TestEqual(TEXT("Inner Object Struct Member Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Inner Object Struct Setter Called"), Test.Object->InnerObject->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Inner Object Struct Getter Called"), Test.Object->InnerObject->IsGetterCalled(), true);
		GPassing &= TestEqual(TEXT("Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Struct Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGetInnerStructMemberInnerObjectTest, "System.PropertyPath.GetInnerStructMemberInnerObjectTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FGetInnerStructMemberInnerObjectTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.Float = CppTest.Object->InnerObject->GetStruct().InnerStruct.Float;

		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("InnerObject.Struct.InnerStruct.Float"), Test.ModifiedStruct.Float);
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Member Get"), Test.DefaultStruct.InnerStruct.Float, Test.ModifiedStruct.Float);
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Member Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Setter Called"), Test.Object->InnerObject->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Getter Called"), Test.Object->InnerObject->IsGetterCalled(), true);
		GPassing &= TestEqual(TEXT("Inner Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Inner Struct Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGetInnerStructMemberEnumInnerObjectTest, "System.PropertyPath.GetInnerStructMemberEnumInnerObjectTest", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FGetInnerStructMemberEnumInnerObjectTest::RunTest(const FString& Parameters)
{
	bool GPassing = true;

	{
		FPropertyPathTestBed Test = {};
		FPropertyPathTestBed CppTest = {};
		CppTest.ModifiedStruct.EnumTwo = CppTest.Object->InnerObject->GetStruct().InnerStruct.EnumTwo;
		PropertyPathHelpers::GetPropertyValue(Test.Object, FString("InnerObject.Struct.InnerStruct.EnumTwo"), Test.ModifiedStruct.EnumTwo);
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Member Enum Get"), Test.DefaultStruct.InnerStruct.EnumTwo, Test.ModifiedStruct.EnumTwo);
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Member Enum Get Cpp Equivalent"), CppTest.ModifiedStruct, Test.ModifiedStruct);
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Setter Called"), Test.Object->InnerObject->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Inner Object Inner Struct Getter Called"), Test.Object->InnerObject->IsGetterCalled(), true);
		GPassing &= TestEqual(TEXT("Inner Struct Setter Called"), Test.Object->IsSetterCalled(), false);
		GPassing &= TestEqual(TEXT("Inner Struct Getter Called"), Test.Object->IsGetterCalled(), false);
		Test.Object->ResetGetterSetterFlags();
	}

	return GPassing;
}

#endif // WITH_DEV_AUTOMATION_TESTS
