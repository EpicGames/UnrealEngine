// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "Misc/AutomationTest.h"
#include "PropertyPathHelpers.h"
#include "PropertyPathHelpersTest.generated.h"

class UPropertyPathTestObject;

UENUM()
enum EPropertyPathTestEnum : int
{
	One,
	Two,
	Three,
	Four
};

USTRUCT()
struct FPropertyPathTestBaseStruct
{
	GENERATED_BODY()

public:
	virtual ~FPropertyPathTestBaseStruct() = default;
};

USTRUCT(BlueprintType)
struct FPropertyPathTestInnerStruct : public FPropertyPathTestBaseStruct
{
	GENERATED_BODY()

	virtual ~FPropertyPathTestInnerStruct()
	{
		Float = -1.5f;
		Bool = true;
		EnumOne = Four;
		EnumTwo = Four;
		EnumThree = Four;
		EnumFour = Four;
		Integer = -1;
		String = "Destructed";
	}

	UPROPERTY()
	float Float = 0.5f;

	UPROPERTY()
	bool Bool = false;

	UPROPERTY()
	TEnumAsByte<EPropertyPathTestEnum> EnumOne = Three;

	UPROPERTY()
	TEnumAsByte<EPropertyPathTestEnum> EnumTwo = Three;

	UPROPERTY()
	TEnumAsByte<EPropertyPathTestEnum> EnumThree = Three;

	UPROPERTY()
	TEnumAsByte<EPropertyPathTestEnum> EnumFour = Three;

	UPROPERTY()
	int32 Integer = 0;

	UPROPERTY()
	FString String = "Default";

	bool operator ==(const FPropertyPathTestInnerStruct& Other) const
	{
		return Bool == Other.Bool
			&& Integer == Other.Integer
			&& EnumOne == Other.EnumOne
			&& EnumTwo == Other.EnumTwo
			&& EnumThree == Other.EnumThree
			&& EnumFour == Other.EnumFour
			&& String == Other.String
			&& Float == Other.Float;
	}

	bool operator !=(const FPropertyPathTestInnerStruct& Other) const
	{
		return !(*this == Other);
	}
};

USTRUCT(BlueprintType)
struct FPropertyPathTestStruct : public FPropertyPathTestBaseStruct
{
	GENERATED_BODY()

	virtual ~FPropertyPathTestStruct()
	{
		Float = -1.5f;
		Bool = true;
		EnumOne = Two;
		EnumTwo = Two;
		EnumThree = Two;
		EnumFour = Two;
		Integer = -1;
		String = "Destructed";
	}

	UPROPERTY()
	bool Bool = false;

	UPROPERTY()
	int32 Integer = 0;

	UPROPERTY()
	TEnumAsByte<EPropertyPathTestEnum> EnumOne = One;

	UPROPERTY()
	TEnumAsByte<EPropertyPathTestEnum> EnumTwo = One;

	UPROPERTY()
	TEnumAsByte<EPropertyPathTestEnum> EnumThree = One;

	UPROPERTY()
	TEnumAsByte<EPropertyPathTestEnum> EnumFour = One;

	UPROPERTY()
	FString String = "Default";

	UPROPERTY()
	float Float = 0.0f;

	UPROPERTY()
	FPropertyPathTestInnerStruct InnerStruct = {};

	UPROPERTY()
	TObjectPtr<UPropertyPathTestObject> InnerObject = nullptr;

	bool operator ==(const FPropertyPathTestStruct& Other) const
	{
		return Bool == Other.Bool
			&& Integer == Other.Integer
			&& EnumOne == Other.EnumOne
			&& EnumTwo == Other.EnumTwo
			&& EnumThree == Other.EnumThree
			&& EnumFour == Other.EnumFour
			&& String == Other.String
			&& Float == Other.Float
			&& InnerStruct == Other.InnerStruct;
	}

	bool operator !=(const FPropertyPathTestStruct& Other) const
	{
		return !(*this == Other);
	}
};

UCLASS()
class UPropertyPathTestObject : public UObject
{
	GENERATED_BODY()
		
public:

	~UPropertyPathTestObject()
	{
		Float = -1.5f;
		Bool = true;
		EnumOne = Two;
		EnumTwo = Two;
		EnumThree = Two;
		EnumFour = Two;
		Integer = -1;
		String = "Destructed";
	}

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "PropertyPathHelpersTest")
	bool Bool = false;

	UPROPERTY(EditDefaultsOnly, Category = "PropertyPathHelpersTest")
	TEnumAsByte<EPropertyPathTestEnum> EnumOne = One;

	UPROPERTY(EditDefaultsOnly, Category = "PropertyPathHelpersTest")
	TEnumAsByte<EPropertyPathTestEnum> EnumTwo = One;

	UPROPERTY(EditDefaultsOnly, Category = "PropertyPathHelpersTest")
	TEnumAsByte<EPropertyPathTestEnum> EnumThree = One;

	UPROPERTY(EditDefaultsOnly, Category = "PropertyPathHelpersTest")
	TEnumAsByte<EPropertyPathTestEnum> EnumFour = One;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "PropertyPathHelpersTest")
	int32 Integer = 0;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = "PropertyPathHelpersTest")
	FString String = "Default";

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Getter, Setter, Category = "PropertyPathHelpersTest")
	float Float = 0.0f;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Getter, Setter, Category = "PropertyPathHelpersTest")
	FPropertyPathTestStruct Struct;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Getter, Setter, Category = "PropertyPathHelpersTest")
	mutable FPropertyPathTestStruct StructRef;

	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Getter, Setter, Category = "PropertyPathHelpersTest")
	FPropertyPathTestStruct StructConstRef;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "PropertyPathHelpersTest")
	TObjectPtr<UPropertyPathTestObject> InnerObject;

public:

	UFUNCTION()
	void SetFloat(float InFloat)
	{
		bSetterCalled = true;
		Float = InFloat;
	}

	UFUNCTION()
	float GetFloat() const
	{
		bGetterCalled = true;
		return Float;
	}

	UFUNCTION()
	void SetStruct(FPropertyPathTestStruct InStruct)
	{
		bSetterCalled = true;
		Struct = InStruct;
	}

	UFUNCTION()
	FPropertyPathTestStruct GetStruct() const
	{
		bGetterCalled = true;
		return Struct;
	}

	UFUNCTION()
	void SetStructRef(FPropertyPathTestStruct InStruct)
	{
		bSetterCalled = true;
		StructRef = InStruct;
	}

	UFUNCTION()
	FPropertyPathTestStruct& GetStructRef() const
	{
		bGetterCalled = true;
		return StructRef;
	}

	UFUNCTION()
	void SetStructConstRef(FPropertyPathTestStruct InStruct)
	{
		bSetterCalled = true;
		StructConstRef = InStruct;
	}

	UFUNCTION()
	const FPropertyPathTestStruct& GetStructConstRef() const
	{
		bGetterCalled = true;
		return StructConstRef;
	}

public:

	bool operator ==(const UPropertyPathTestObject& Other) const
	{
		return Bool == Other.Bool
			&& Integer == Other.Integer
			&& EnumOne == Other.EnumOne
			&& EnumTwo == Other.EnumTwo
			&& EnumThree == Other.EnumThree
			&& EnumFour == Other.EnumFour
			&& String == Other.String
			&& Float == Other.Float
			&& Struct == Other.Struct
			&& StructRef == Other.StructRef
			&& StructConstRef == Other.StructConstRef
			&& bSetterCalled == Other.bSetterCalled
			&& bGetterCalled == Other.bGetterCalled
			&& InnerObject ? (*InnerObject == *Other.InnerObject) : true;
	}

	bool operator !=(const UPropertyPathTestObject& Other) const
	{
		return !(*this == Other);
	}

	bool IsSetterCalled() const
	{
		return bSetterCalled;
	}

	bool IsGetterCalled() const
	{
		return bGetterCalled;
	}
		
	void ResetGetterSetterFlags() const
	{
		bSetterCalled = false;
		bGetterCalled = false;
	}

private:

	mutable bool bSetterCalled = false;
	mutable bool bGetterCalled = false;
};

USTRUCT()
struct FPropertyPathTestBed
{
	GENERATED_BODY()

	FPropertyPathTestBed()
	{
		Object = NewObject<UPropertyPathTestObject>();
		Object->InnerObject = NewObject<UPropertyPathTestObject>();
		Object->Struct.InnerObject = NewObject<UPropertyPathTestObject>();
		Object->StructRef.InnerObject = NewObject<UPropertyPathTestObject>();
		Object->StructConstRef.InnerObject = NewObject<UPropertyPathTestObject>();

		ModifiedStruct = {};

		ModifiedStruct.Bool = true;
		ModifiedStruct.Integer = 1;
		ModifiedStruct.EnumOne = Two;
		ModifiedStruct.EnumTwo = Two;
		ModifiedStruct.EnumThree = Two;
		ModifiedStruct.EnumFour = Two;
		ModifiedStruct.String = "NewValue";
		ModifiedStruct.Float = 1.5f;

		ModifiedStruct.InnerStruct.Bool = true;
		ModifiedStruct.InnerStruct.Integer = 1;
		ModifiedStruct.EnumOne = Four;
		ModifiedStruct.EnumTwo = Four;
		ModifiedStruct.EnumThree = Four;
		ModifiedStruct.EnumFour = Four;
		ModifiedStruct.InnerStruct.String = "NewValue";
		ModifiedStruct.InnerStruct.Float = 1.5f;

		DefaultStruct = {};

		ModifiedObject = NewObject<UPropertyPathTestObject>();

		ModifiedObject->Bool = true;
		ModifiedObject->Integer = 1;
		ModifiedObject->EnumOne = Two;
		ModifiedObject->EnumTwo = Two;
		ModifiedObject->EnumThree = Two;
		ModifiedObject->EnumFour = Two;
		ModifiedObject->String = "NewValue";
		ModifiedObject->Float = 1.5f;
	}

	UPROPERTY()
	TObjectPtr<UPropertyPathTestObject> Object;

	UPROPERTY()
	TObjectPtr<UPropertyPathTestObject> ModifiedObject;

	UPROPERTY()
	FPropertyPathTestStruct ModifiedStruct;

	UPROPERTY()
	FPropertyPathTestStruct DefaultStruct;
};
