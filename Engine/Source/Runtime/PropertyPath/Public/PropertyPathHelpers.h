// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "PropertyTypeCompatibility.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealType.h"

#include "PropertyPathHelpers.generated.h"

class FArchive;
template <typename FuncType> class TFunctionRef;

/** Base class for cached property path segments */
USTRUCT()
struct FPropertyPathSegment
{
	GENERATED_BODY()

	friend struct FCachedPropertyPath;

	/** UStruct boilerplate constructor - do not use */
	PROPERTYPATH_API FPropertyPathSegment();

	/** Construct from char count and storage */
	PROPERTYPATH_API FPropertyPathSegment(int32 InCount, const TCHAR* InString);

	/**
	 * Called after this event has been serialized in order to cache the field pointer if necessary
	 */
	PROPERTYPATH_API void PostSerialize(const FArchive& Ar);

	/** Make a copy which is unresolved */
	static PROPERTYPATH_API FPropertyPathSegment MakeUnresolvedCopy(const FPropertyPathSegment& ToCopy);

	/**
	 * Resolves the name on the given Struct.  Can be used to cache the resulting property so that future calls can be processed quickly.
	 * @param InStruct the ScriptStruct or Class to look for the property on.
	 */
	PROPERTYPATH_API FFieldVariant Resolve(UStruct* InStruct) const;

	/** @return the name of this segment */
	PROPERTYPATH_API FName GetName() const;

	/** @return the array index of this segment */
	PROPERTYPATH_API int32 GetArrayIndex() const;

	/** @return the resolved field */
	PROPERTYPATH_API FFieldVariant GetField() const;

	/** @return the resolved struct */
	PROPERTYPATH_API UStruct* GetStruct() const;

public:

	/** The sub-component of the property path, a single value between .'s of the path */
	UPROPERTY()
	FName Name;

	/** The optional array index. */
	UPROPERTY()
	int32 ArrayIndex;

private:

	/** The cached Class or ScriptStruct that was used last to resolve Name to a property. */
	UPROPERTY(Transient)
	mutable TObjectPtr<UStruct> Struct;

	/**
	 * The cached property on the Struct that this Name resolved to on it last time Resolve was called, if 
	 * the Struct doesn't change, this value is returned to avoid performing another Field lookup.
	 */
	mutable FFieldVariant Field;
};

template<>
struct TStructOpsTypeTraits<FPropertyPathSegment> : TStructOpsTypeTraitsBase2<FPropertyPathSegment>
{
	enum { WithPostSerialize = true };
};

/** Base class for cached property paths */
USTRUCT()
struct FCachedPropertyPath
{
	GENERATED_BODY()

	/** UStruct boilerplate constructor - do not use */
	PROPERTYPATH_API FCachedPropertyPath();

	/** */
	PROPERTYPATH_API FCachedPropertyPath(const FString& Path);

	/** */
	PROPERTYPATH_API FCachedPropertyPath(const FPropertyPathSegment& Segment);

	/** */
	PROPERTYPATH_API FCachedPropertyPath(const TArray<FString>& PathSegments);

	/** Allows constructing a cached property path from the standard events of a PostEditChange Notify or Callback */
	PROPERTYPATH_API FCachedPropertyPath(const FPropertyChangedChainEvent& PropertyChangedChainEvent);
	PROPERTYPATH_API FCachedPropertyPath(const FPropertyChangedEvent& PropertyChangedEvent, const FEditPropertyChain& PropertyChain);

	/** */
	PROPERTYPATH_API ~FCachedPropertyPath();

	/** Check whether this property path is non-empty */
	bool IsValid() const { return Segments.Num() > 0; }

	/** Make a new property path from a string */
	PROPERTYPATH_API void MakeFromString(const FString& InPropertyPath);

	/** Make a new property path from the chain of properties that come standard in PostEditPropertyChain events */
	PROPERTYPATH_API void MakeFromChangeEvent(const FPropertyChangedChainEvent& PropertyChangedChainEvent);
	PROPERTYPATH_API void MakeFromChangeEvent(const FPropertyChangedEvent& PropertyChangedEvent, const FEditPropertyChain& PropertyChain);

	/** Make a copy which is unresolved */
	static PROPERTYPATH_API FCachedPropertyPath MakeUnresolvedCopy(const FCachedPropertyPath& ToCopy);

	/** Make a copy without the last segment, if the path is resolved, the new path is also resolved. */
	PROPERTYPATH_API FCachedPropertyPath MakeParentPath() const;

	/** @return Get the number of segments in this path */
	PROPERTYPATH_API int32 GetNumSegments() const;

	/** 
	 * Get the path segment at the specified index
	 * @param	InSegmentIndex	The index of the segment
	 * @return the segment at the specified index 
	 */
	PROPERTYPATH_API const FPropertyPathSegment& GetSegment(int32 InSegmentIndex) const;

	/** 
	 * Get the path segment at the end of the path
	 * @return the segment at the specified index 
	 */
	PROPERTYPATH_API const FPropertyPathSegment& GetLastSegment() const;

	/** 
	 * Resolve this property path against the specified object.
	 * @return true if the path could be resolved
	 */
	PROPERTYPATH_API bool Resolve(UObject* InContainer) const;

	/** Set whether this path resolves over object or dynamic array boundaries, making it unsafe for general direct cached access */
	PROPERTYPATH_API void SetCanSafelyUsedCachedAddress(bool bInCanSafelyUsedCachedAddress) const;

	/** Update cached last container property in path & correspondng index, invalidates 'bCanSafelyUsedCachedAddress' */
	PROPERTYPATH_API void SetCachedLastContainer(void* InContainer, int32 InIndex) const;

	/** Get cached last container property in path */
	PROPERTYPATH_API void* GetCachedLastContainerInPath() const;

	/** Get cached index of last container property in path, INDEX_NONE if last container is not in path */
	PROPERTYPATH_API int32 GetCachedLastContainerInPathIndex() const;

	/** Cache a resolved address for faster subsequent access */
	PROPERTYPATH_API void ResolveLeaf(void* InAddress) const;

	/** Cache a resolved function for faster subsequent access */
	PROPERTYPATH_API void ResolveLeaf(UFunction* InFunction) const;

	/** 
	 * Check whether a path is resolved. This means that it has a cached address, but may
	 * resolve over an object boundary or a dynamic array.
	 * @return true if the path is resolved
	 */
	PROPERTYPATH_API bool IsResolved() const;

	/** 
	 * Check whether a path is fully resolved. This means that it has a cached address and
	 * does not resolve over an object boundary or a dynamic array.
	 * @return true if the path is fully resolved
	 */
	PROPERTYPATH_API bool IsFullyResolved() const;

	/** Get the cached address for this property path, if any */
	PROPERTYPATH_API void* GetCachedAddress() const;

	/** Get the cached function for this property path, if any */
	PROPERTYPATH_API UFunction* GetCachedFunction() const;

	/** Convert this property path to a FPropertyChangedEvent. Note that the path must be resolved. */
	PROPERTYPATH_API FPropertyChangedEvent ToPropertyChangedEvent(EPropertyChangeType::Type InChangeType) const;

	/** Convert this property path to a FEditPropertyChain. Note that the path must be resolved. */
	PROPERTYPATH_API void ToEditPropertyChain(FEditPropertyChain& OutPropertyChain) const;

	/** Make a string representation of this property path */
	PROPERTYPATH_API FString ToString() const;

	/** Compares this property path to a string */
	PROPERTYPATH_API bool operator==(const FString& Other) const;

	/** Compares this property path to a string */
	PROPERTYPATH_API bool Equals(const FString& Other) const;

	/** Get the cached container for this property path, for checking purposes */
	PROPERTYPATH_API void* GetCachedContainer() const;

	/** Set the cached container for this property path, for checking purposes */
	PROPERTYPATH_API void SetCachedContainer(void* InContainer) const;

	/** Trims this property path at the end */
	PROPERTYPATH_API void RemoveFromEnd(int32 InNumSegments = 1);

	/** Trims this property path at the start */
	PROPERTYPATH_API void RemoveFromStart(int32 InNumSegments = 1);

	/** Returns FProperty if valid. This can be UFunction */
	PROPERTYPATH_API FProperty* GetFProperty() const;
private:
	/** Path segments for this path */
	UPROPERTY()
	TArray<FPropertyPathSegment> Segments;

	/** Cached read/write address for property-terminated paths */
	mutable void* CachedAddress;

	/** Cached function for function-terminated paths */
	UPROPERTY()
	mutable TObjectPtr<UFunction> CachedFunction;

	/** Cached container */
	mutable void* CachedContainer;

	/** Cached last container */
	mutable void* CachedLastContainerInPath;

	/** Index of last container property in path, INDEX_NONE if last container is not in path */
	mutable int32 CachedLastContainerInPathIndex;

	/** Whether this path resolves over object or dynamic array boundaries, making it unsafe for general direct cached access */
	mutable bool bCanSafelyUsedCachedAddress;
};

/** Forward declarations of internals */
namespace PropertyPathHelpersInternal
{
	struct FPropertyPathResolver;
	struct FPropertyStructView;
	template<typename T> struct FInternalGetterResolver;
	template<typename T> struct FInternalSetterResolver;

	PROPERTYPATH_API bool ResolvePropertyPath(UObject* InContainer, const FString& InPropertyPath, FPropertyPathResolver& InResolver);
	PROPERTYPATH_API bool ResolvePropertyPath(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, FPropertyPathResolver& InResolver);
	PROPERTYPATH_API bool ResolvePropertyPath(void* InContainer, UStruct* InStruct, const FString& InPropertyPath, FPropertyPathResolver& InResolver);
	PROPERTYPATH_API bool ResolvePropertyPath(void* InContainer, UStruct* InStruct, const FCachedPropertyPath& InPropertyPath, FPropertyPathResolver& InResolver);
	template<typename T, typename ContainerType> bool GetValueFast(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath, T& OutValue, FProperty*& OutProperty);
	template<typename T, typename ContainerType> bool SetValueFast(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath, const T& InValue);
}

/** A collection of utility functions operating on cached property paths */
namespace PropertyPathHelpers
{
	/** 
	 * Get the value represented by this property path as a string 
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path string
	 * @param	OutValue		The string to write the properties value to
	 * @return true if the property value was successfully copied
	 */
	PROPERTYPATH_API bool GetPropertyValueAsString(UObject* InContainer, const FString& InPropertyPath, FString& OutValue, uint32 PPF_Flags = PPF_None);

	/** 
	 * Get the value represented by this property path as a string 
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path string
	 * @param	OutValue		The string to write the properties value to
	 * @param	OutProperty		The leaf property that the path resolved to
	 * @return true if the property value was successfully copied
	 */
	PROPERTYPATH_API bool GetPropertyValueAsString(UObject* InContainer, const FString& InPropertyPath, FString& OutValue, FProperty*& OutProperty, uint32 PPF_Flags = PPF_None);

	/** 
	 * Get the value represented by this property path as a string 
	 * @param	InContainer		A pointer to the container structure to resolve the property path against
	 * @param	InStruct		The struct type that InContainer points to
	 * @param	InPropertyPath	The property path string
	 * @param	OutValue		The string to write the properties value to
	 * @return true if the property value was successfully copied
	 */
	PROPERTYPATH_API bool GetPropertyValueAsString(void* InContainer, UStruct* InStruct, const FString& InPropertyPath, FString& OutValue, uint32 PPF_Flags = PPF_None);

	/** 
	 * Get the value represented by this property path as a string 
	 * @param	InContainer		A pointer to the container structure to resolve the property path against
	 * @param	InStruct		The struct type that InContainer points to
	 * @param	InPropertyPath	The property path string
	 * @param	OutValue		The string to write the properties value to
	 * @param	OutProperty		The leaf property that the path resolved to
	 * @return true if the property value was successfully copied
	 */
	PROPERTYPATH_API bool GetPropertyValueAsString(void* InContainer, UStruct* InStruct, const FString& InPropertyPath, FString& OutValue, FProperty*& OutProperty, uint32 PPF_Flags = PPF_None);

	/** 
	 * Get the value represented by this property path as a string 
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path
	 * @param	OutValue		The string to write the properties value to
	 * @return true if the property value was successfully copied
	 */
	PROPERTYPATH_API bool GetPropertyValueAsString(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, FString& OutValue, uint32 PPF_Flags = PPF_None);

	/** 
	 * Get the value represented by this property path as a string 
	 * @param	InContainer		A pointer to the container structure to resolve the property path against
	 * @param	InStruct		The struct type that InContainer points to
	 * @param	InPropertyPath	The property path
	 * @param	OutValue		The string to write the properties value to
	 * @return true if the property value was successfully copied
	 */
	PROPERTYPATH_API bool GetPropertyValueAsString(void* InContainer, UStruct* InStruct, const FCachedPropertyPath& InPropertyPath, FString& OutValue, uint32 PPF_Flags = PPF_None);

	/** 
	 * Set the value represented by this property path from a string 
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path string
	 * @param	InValue			The string to read the properties value from
	 * @return true if the property value was successfully copied
	 */
	PROPERTYPATH_API bool SetPropertyValueFromString(UObject* InContainer, const FString& InPropertyPath, const FString& InValue, uint32 PPF_Flags = PPF_None);

	/** 
	 * Set the value represented by this property path from a string 
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path
	 * @param	InValue			The string to read the properties value from
	 * @return true if the property value was successfully copied
	 */
	PROPERTYPATH_API bool SetPropertyValueFromString(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, const FString& InValue, uint32 PPF_Flags = PPF_None);

	/** 
	 * Set the value represented by this property path from a string 
	 * @param	InContainer		A pointer to the container structure to resolve the property path against
	 * @param	InStruct		The struct type that InContainer points to
	 * @param	InPropertyPath	The property path string
	 * @param	InValue			The string to read the properties value from
	 * @return true if the property value was successfully copied
	 */
	PROPERTYPATH_API bool SetPropertyValueFromString(void* InContainer, UStruct* InStruct, const FString& InPropertyPath, const FString& InValue, uint32 PPF_Flags = PPF_None);

	/** 
	 * Set the value represented by this property path from a string 
	 * @param	InContainer		A pointer to the container structure to resolve the property path against
	 * @param	InStruct		The struct type that InContainer points to
	 * @param	InPropertyPath	The property path
	 * @param	InValue			The string to read the properties value from
	 * @return true if the property value was successfully copied
	 */
	PROPERTYPATH_API bool SetPropertyValueFromString(void* InContainer, UStruct* InStruct, const FCachedPropertyPath& InPropertyPath, const FString& InValue, uint32 PPF_Flags = PPF_None);

	/** 
	 * Get the value represented by this property path. forcing the use of cached addresses whether or
	 * not the path resolves over object or dynamic array boundaries.
	 * Using this function implies that the path is resolved and has not changed since last resolution.
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path
	 * @param	OutValue		The value to write to
	 * @return true if the property value was successfully copied
	 */
	template<typename T>
	bool GetPropertyValueFast(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, T& OutValue)
	{
		FProperty* OutProperty;
		return GetPropertyValueFast<T>(InContainer, InPropertyPath, OutValue, OutProperty);
	}

	/** 
	 * Get the value and the leaf property represented by this property path, forcing the use of 
	 * cached addresses whether or not the path resolves over object or dynamic array boundaries.
	 * Using this function implies that the path is resolved and has not changed since last resolution.
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path
	 * @param	OutValue		The value to write to
	 * @param	OutProperty		The leaf property that the path resolved to
	 * @return true if the property value was successfully copied
	 */
	template<typename T>
	bool GetPropertyValueFast(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, T& OutValue, FProperty*& OutProperty)
	{
		check(InContainer);
		check(InContainer == InPropertyPath.GetCachedContainer());
		check(InPropertyPath.IsResolved());

		return PropertyPathHelpersInternal::GetValueFast<T>(InContainer, InPropertyPath, OutValue, OutProperty);
	}

	/** 
	 * Get the value and the leaf property represented by this property path.
	 * If the cached property path has a cached address it will use that as a 'fast path' instead 
	 * of iterating the path. This has safety implications depending on the form of the path, so 
	 * paths that are resolved over object boundaries or dynamic arrays will always use the slow 
	 * path for safety.
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path
	 * @param	OutValue		The value to write to
	 * @param	OutProperty		The leaf property that the path resolved to
	 * @return true if the property value was successfully copied
	 */
	template<typename T>
	bool GetPropertyValue(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, T& OutValue, FProperty*& OutProperty)
	{
		check(InContainer);

		if(InPropertyPath.IsFullyResolved())
		{
			return GetPropertyValueFast<T>(InContainer, InPropertyPath, OutValue, OutProperty);
		}
		else
		{
			PropertyPathHelpersInternal::FInternalGetterResolver<T> Resolver(OutValue, OutProperty);
			return PropertyPathHelpersInternal::ResolvePropertyPath(InContainer, InPropertyPath, Resolver);
		}
	}

	/** 
	 * Get the value represented by this property path.
	 * If the cached property path has a cached address it will use that as a 'fast path' instead 
	 * of iterating the path. This has safety implications depending on the form of the path, so 
	 * paths that are resolved over object boundaries or dynamic arrays will always use the slow 
	 * path for safety.
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path string
	 * @param	OutValue		The value to write to
	 * @return true if the property value was successfully copied
	 */
	template<typename T>
	bool GetPropertyValue(UObject* InContainer, const FString& InPropertyPath, T& OutValue)
	{
		FProperty* OutProperty;
		FCachedPropertyPath CachedPath(InPropertyPath);
		return GetPropertyValue<T>(InContainer, CachedPath, OutValue, OutProperty);
	}

	/** 
	 * Get the value and the leaf property represented by this property path.
	 * If the cached property path has a cached address it will use that as a 'fast path' instead 
	 * of iterating the path. This has safety implications depending on the form of the path, so 
	 * paths that are resolved over object boundaries or dynamic arrays will always use the slow 
	 * path for safety.
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path string
	 * @param	OutValue		The value to write to
	 * @param	OutProperty		The leaf property that the path resolved to
	 * @return true if the property value was successfully copied
	 */
	template<typename T>
	bool GetPropertyValue(UObject* InContainer, const FString& InPropertyPath, T& OutValue, FProperty*& OutProperty)
	{
		FCachedPropertyPath CachedPath(InPropertyPath);
		return GetPropertyValue<T>(InContainer, CachedPath, OutValue, OutProperty);
	}

	/** 
	 * Get the value represented by this property path.
	 * If the cached property path has a cached address it will use that as a 'fast path' instead 
	 * of iterating the path. This has safety implications depending on the form of the path, so 
	 * paths that are resolved over object boundaries or dynamic arrays will always use the slow 
	 * path for safety.
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path
	 * @param	OutValue		The value to write to
	 * @return true if the property value was successfully copied
	 */
	template<typename T>
	bool GetPropertyValue(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, T& OutValue)
	{
		FProperty* OutProperty;
		return GetPropertyValue<T>(InContainer, InPropertyPath, OutValue, OutProperty);
	}

	/** 
	 * Get the value represented by this property path. Fast, unsafe version. 
	 * Using this function implies that the path is resolved and has not changed since last resolution.
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path
	 * @param	InValue			The value to write from
	 * @return true if the property value was successfully copied
	 */
	template<typename T>
	bool SetPropertyValueFast(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, const T& InValue)
	{
		check(InContainer);
		check(InContainer == InPropertyPath.GetCachedContainer());
		check(InPropertyPath.IsResolved());

		return PropertyPathHelpersInternal::SetValueFast<T>(InContainer, InPropertyPath, InValue);
	}

	/** 
	 * Set the value and the leaf property represented by this property path 
	 * If the cached property path has a cached address it will use that as a 'fast path' instead 
	 * of iterating the path. This has safety implications depending on the form of the path, so 
	 * paths that are resolved over object boundaries or dynamic arrays will always use the slow 
	 * path for safety.
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path
	 * @param	InValue			The value to write from
	 * @return true if the property value was successfully copied
	 */
	template<typename T>
	bool SetPropertyValue(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, const T& InValue)
	{
		check(InContainer);

		if(InPropertyPath.IsFullyResolved())
		{
			return SetPropertyValueFast<T>(InContainer, InPropertyPath, InValue);
		}
		else
		{
			PropertyPathHelpersInternal::FInternalSetterResolver<T> Resolver(InValue);
			return PropertyPathHelpersInternal::ResolvePropertyPath(InContainer, InPropertyPath, Resolver);
		}
	}

	/** 
	 * Set the value and the leaf property represented by this property path 
	 * If the cached property path has a cached address it will use that as a 'fast path' instead 
	 * of iterating the path. This has safety implications depending on the form of the path, so 
	 * paths that are resolved over object boundaries or dynamic arrays will always use the slow 
	 * path for safety.
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path string
	 * @param	InValue			The value to write from
	 * @return true if the property value was successfully copied
	 */
	template<typename T>
	bool SetPropertyValue(UObject* InContainer, const FString& InPropertyPath, const T& InValue)
	{
		FCachedPropertyPath CachedPath(InPropertyPath);
		return SetPropertyValue(InContainer, CachedPath, InValue);
	}

	/** 
	 * Set the value and the leaf property represented by this property path 
	 * If the cached property path has a cached address it will use that as a 'fast path' instead 
	 * of iterating the path. This has safety implications depending on the form of the path, so 
	 * paths that are resolved over object boundaries or dynamic arrays will always use the slow 
	 * path for safety.
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path
	 * @param	InScriptStruct	The struct type to set
	 * @param	InValue			A pointer to the desired value for the given struct type
	 * @return true if the property value was successfully copied
	 */
	PROPERTYPATH_API bool SetPropertyValue(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, const UScriptStruct* InScriptStruct, const uint8* InValue);

	/** 
	 * Set the value and the leaf property represented by this property path 
	 * If the cached property path has a cached address it will use that as a 'fast path' instead 
	 * of iterating the path. This has safety implications depending on the form of the path, so 
	 * paths that are resolved over object boundaries or dynamic arrays will always use the slow 
	 * path for safety.
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path string
	 * @param	InScriptStruct	The struct type to set
	 * @param	InValue			A pointer to the desired value for the given struct type
	 * @return true if the property value was successfully copied
	 */
	PROPERTYPATH_API bool SetPropertyValue(UObject* InContainer, const FString& InPropertyPath, const UScriptStruct* InScriptStruct, const uint8* InValue);

	/** 
	 * Copy values between two property paths in the same container.
	 * @param	InContainer			The container object to resolve the property path against
	 * @param	InDestPropertyPath	The property path to copy to
	 * @param	InSrcPropertyPath	The property path to copy from
	 * @return true if the copy was successful
	 */
	PROPERTYPATH_API bool CopyPropertyValue(UObject* InContainer, const FCachedPropertyPath& InDestPropertyPath, const FCachedPropertyPath& InSrcPropertyPath);

	/** 
	 * Copy values between two property paths in the same container. Fast, unsafe version.
	 * Using this requires that the two paths are pre-resolved from either a previous call
	 * to CopyPropertyValue (or GetPropertyValue/SetPropertyValue) or 
	 * FCachedPropertyPath::Resolve().
	 * @param	InContainer			The container object to resolve the property path against
	 * @param	InDestPropertyPath	The property path to copy to
	 * @param	InSrcPropertyPath	The property path to copy from
	 * @return true if the copy was successful
	 */
	PROPERTYPATH_API bool CopyPropertyValueFast(UObject* InContainer, const FCachedPropertyPath& InDestPropertyPath, const FCachedPropertyPath& InSrcPropertyPath);

	/** 
	 * Parses a property path segment name of the form PropertyName[OptionalArrayIndex]
	 * @param	InCount			The length of the input string
	 * @param	InString        The Input string to parse
	 * @param	OutCount		The length of the resulting property name string
	 * @param   OutPropertyName The string storing the name of the property
	 * @param	OutArrayIndex	The resulting array index, if any
	 */
	PROPERTYPATH_API void FindFieldNameAndArrayIndex(int32 InCount, const TCHAR* InString, int32& OutCount, const TCHAR** OutPropertyName, int32& OutArrayIndex);

	/** 
	 * Perform the specified operation on the array referenced by the property path
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path string
	 * @param	InOperation		The operation to perform. The function will receive a script array helper and an index (if the path specified one).
	 * @return true if the operation was successful
	 */
	PROPERTYPATH_API bool PerformArrayOperation(UObject* InContainer, const FString& InPropertyPath, TFunctionRef<bool(FScriptArrayHelper&,int32)> InOperation);

	/** 
	 * Perform the specified operation on the array referenced by the property path
	 * @param	InContainer		The container object to resolve the property path against
	 * @param	InPropertyPath	The property path
	 * @param	InOperation		The operation to perform. The function will receive a script array helper and an index (if the path specified one).
	 * @return true if the operation was successful
	 */
	PROPERTYPATH_API bool PerformArrayOperation(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, TFunctionRef<bool(FScriptArrayHelper&,int32)> InOperation);
}

/** Internal implementations of property path helpers - do not use directly */
namespace PropertyPathHelpersInternal
{
	/** Base class for resolving property paths */
	struct FPropertyPathResolver
	{
		/** Virtual destructor */
		virtual ~FPropertyPathResolver() = default;

		/** 
		 * Callbacks for a leaf segment in a property path iteration. Used to resolve the property path.
		 * @param InContainer			The containing object/structure for the current path iteration
		 * @param InPropertyPath		The property path to resolve
		 * @return true if the iteration should continue.
		 */
		virtual bool Resolve(void* InContainer, const FCachedPropertyPath& InPropertyPath) = 0;
		virtual bool Resolve(UObject* InContainer, const FCachedPropertyPath& InPropertyPath) =  0;
	};

	/** Helper struct to represent a view of a struct, used to specialize behavior against */
	struct FPropertyStructView
	{
		explicit FPropertyStructView(const UScriptStruct* InScriptStruct, const uint8* InMemory)
			: ScriptStruct(InScriptStruct)
			, Memory(InMemory)
		{
		}

		const UScriptStruct* ScriptStruct;
		const uint8* Memory;
	};

	/** Recurring template allowing derived types to only implement templated Resolve_Impl */
	template<typename DerivedType>
	struct TPropertyPathResolver : public FPropertyPathResolver
	{
		virtual bool Resolve(void* InContainer, const FCachedPropertyPath& InPropertyPath) override
		{
			return static_cast<DerivedType*>(this)->template Resolve_Impl<void>(InContainer, InPropertyPath);
		}

		virtual bool Resolve(UObject* InContainer, const FCachedPropertyPath& InPropertyPath) override
		{
			return static_cast<DerivedType*>(this)->template Resolve_Impl<UObject>(InContainer, InPropertyPath);
		}
	};

	/** Find the first param that isnt a return property for the specified function */
	PROPERTYPATH_API FProperty* GetFirstParamProperty(UFunction* InFunction);

	/** Helper function used to call parent setters, used when modifying elements without setters in a struct that has one */
	PROPERTYPATH_API void CallParentSetters(const FCachedPropertyPath& InPropertyPath);

	/** 
	 * Helper function used to call parent getters, used when modifying elements without getters in a struct that has one 
	 * 
	 * @return True if a parent getter was called.
	 */
	PROPERTYPATH_API void CallParentGetters(void* OutValue, const FCachedPropertyPath& InPropertyPath, const void* InPropertyAddress);

	/** Non-UObject helper struct for GetValue function calls */
	template<typename T, typename ContainerType>
	struct FCallGetterFunctionHelper
	{
		static bool CallGetterFunction(ContainerType* InContainer, UFunction* InFunction, T& OutValue) 
		{
			// Cant call UFunctions on non-UObject containers
			return false;
		}
	};

	/** UObject partial specialization of FCallGetterFunctionHelper */
	template<typename T>
	struct FCallGetterFunctionHelper<T, UObject>
	{
		static bool CallGetterFunction(UObject* InContainer, UFunction* InFunction, T& OutValue) 
		{
			// We only support calling functions that return a single value and take no parameters.
			if ( InFunction->NumParms == 1 )
			{
				// Find the pointer to the only param
				FProperty* ReturnProperty = nullptr;
				for (TFieldIterator<FProperty> It(InFunction); It; ++It)
				{
					if (It->PropertyFlags & CPF_Parm)
					{
						ReturnProperty = *It;
						break;
					}
				}

				// Verify there's a return property.
				if ( ReturnProperty && ReturnProperty->PropertyFlags & (CPF_ReturnParm | CPF_OutParm) )
				{
					// Verify that the cpp type matches a known property type.
					if ( IsConcreteTypeCompatibleWithReflectedType<T>(ReturnProperty) )
					{
						// Ensure that the element sizes are the same, prevents the user from doing something terribly wrong.
						if ( PropertySizesMatch<T>(ReturnProperty) && !InContainer->IsUnreachable() )
						{
							InContainer->ProcessEvent(InFunction, &OutValue);
							return true;
						}
					}
				}
			}

			return false;
		}
	};

	/** Helper function used to get a value */
	template<typename T, typename ContainerType>
	struct FGetValueHelper
	{
		static bool GetValue(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath, T& OutValue, FProperty*& OutProperty)
		{
			const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
			int32 ArrayIndex = LastSegment.GetArrayIndex();
			FProperty* Property = CastFieldChecked<FProperty>(LastSegment.GetField().ToField());

			// Verify that the cpp type matches a known property type.
			if ( IsConcreteTypeCompatibleWithReflectedType<T>(Property) )
			{
				ArrayIndex = ArrayIndex == INDEX_NONE ? 0 : ArrayIndex;
				if (PropertySizesMatch<T>(Property) && ArrayIndex < Property->ArrayDim)
				{
					if (Property->HasGetter())
					{
						Property->CallGetter(InContainer, &OutValue);
					}
					else if (void* Address = Property->ContainerPtrToValuePtr<T>(InContainer, ArrayIndex))
					{
						InPropertyPath.ResolveLeaf(Address);
						CallParentGetters(&OutValue, InPropertyPath, Address);
					}
					OutProperty = Property;
					return true;
				}
			}

			return false;
		}
	};

	/** Partial specialization for arrays */
	template<typename T, typename ContainerType, int32 N>
	struct FGetValueHelper<T[N], ContainerType>
	{
		static bool GetValue(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath, T(&OutValue)[N], FProperty*& OutProperty)
		{
			const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
			FProperty* Property = CastFieldChecked<FProperty>(LastSegment.GetField().ToField());

			// Verify that the cpp type matches a known property type.
			if ( IsConcreteTypeCompatibleWithReflectedType<T>(Property) )
			{
				if ( PropertySizesMatch<T>(Property) && Property->ArrayDim == N )
				{
					if(void* Address = Property->ContainerPtrToValuePtr<T>(InContainer))
					{
						InPropertyPath.ResolveLeaf(Address);
						Property->CopyCompleteValue(&OutValue, Address);
						OutProperty = Property;
						return true;
					}
				}
			}

			return true;
		}
	};

	/** Partial specialization for bools/bitfields */
	template<typename ContainerType>
	struct FGetValueHelper<bool, ContainerType>
	{
		static bool GetValue(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath, bool& OutValue, FProperty*& OutProperty)
		{
			const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
			int32 ArrayIndex = LastSegment.GetArrayIndex();
			FProperty* Property = CastFieldChecked<FProperty>(LastSegment.GetField().ToField());

			// Verify that the cpp type matches a known property type.
			if ( IsConcreteTypeCompatibleWithReflectedType<bool>(Property) )
			{
				ArrayIndex = ArrayIndex == INDEX_NONE ? 0 : ArrayIndex;
				if ( PropertySizesMatch<bool>(Property) && ArrayIndex < Property->ArrayDim )
				{
					if(void* Address = Property->ContainerPtrToValuePtr<bool>(InContainer, ArrayIndex))
					{
						InPropertyPath.ResolveLeaf(Address);
						FBoolProperty* BoolProperty = CastFieldChecked<FBoolProperty>(LastSegment.GetField().ToField());
						OutValue = BoolProperty->GetPropertyValue(Address);
						OutProperty = Property;
						return true;
					}
				}
			}

			return false;
		}
	};

	/** 
	 * Resolve a property path to a property and a value. Supports functions as input fields.
	 * @param InContainer			The containing object/structure for the current path iteration
	 * @param InPropertyPath		The property path to get from
	 * @param OutProperty			The resolved property
	 * @param OutValue				The resolved value
	 * @return true if the address and property were resolved
	 */
	template<typename T, typename ContainerType>
	bool GetValue(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath, FProperty*& OutProperty, T& OutValue)
	{
		const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
		int32 ArrayIndex = LastSegment.GetArrayIndex();
		FFieldVariant Field = LastSegment.GetField();

		// We're on the final property in the path, it may be an array property, so check that first.
		if ( FArrayProperty* ArrayProp = Field.Get<FArrayProperty>() )
		{
			// if it's an array property, we need to see if we parsed an index as part of the segment
			// as a user may have baked the index directly into the property path.
			if(ArrayIndex != INDEX_NONE)
			{
				// Property is an array property, so make sure we have a valid index specified
				FScriptArrayHelper_InContainer ArrayHelper(ArrayProp, InContainer);
				if ( ArrayHelper.IsValidIndex(ArrayIndex) )
				{
					// Verify that the cpp type matches a known property type.
					if ( IsConcreteTypeCompatibleWithReflectedType<T>(ArrayProp->Inner) )
					{
						// Ensure that the element sizes are the same, prevents the user from doing something terribly wrong.
						if ( PropertySizesMatch<T>(ArrayProp->Inner) )
						{
							if(void* Address = static_cast<void*>(ArrayHelper.GetRawPtr(ArrayIndex)))
							{
								InPropertyPath.ResolveLeaf(Address);
								ArrayProp->Inner->CopySingleValue(&OutValue, Address);
								OutProperty = ArrayProp->Inner;
								return true;
							}
						}
					}
				}
			}
			else
			{
				// No index, so assume we want the array property itself
				if ( IsConcreteTypeCompatibleWithReflectedType<T>(ArrayProp) )
				{
					if ( PropertySizesMatch<T>(ArrayProp) )
					{
						if(void* Address = ArrayProp->ContainerPtrToValuePtr<T>(InContainer))
						{
							InPropertyPath.ResolveLeaf(Address);
							ArrayProp->CopySingleValue(&OutValue, Address);
							OutProperty = ArrayProp;
							return true;
						}
					}
				}
			}
		}
		else if(UFunction* Function = Field.Get<UFunction>())
		{
			InPropertyPath.ResolveLeaf(Function);
			return FCallGetterFunctionHelper<T, ContainerType>::CallGetterFunction(InContainer, Function, OutValue);
		}
		else if(FProperty* Property = Field.Get<FProperty>())
		{
			return FGetValueHelper<T, ContainerType>::GetValue(InContainer, InPropertyPath, OutValue, OutProperty);
		}

		return false;
	}

	template<typename T>
	struct FInternalGetterResolver : public TPropertyPathResolver<FInternalGetterResolver<T>>
	{
		FInternalGetterResolver(T& InValue, FProperty*& InOutProperty)
			: Value(InValue)
			, Property(InOutProperty)
		{
		}

		template<typename ContainerType>
		bool Resolve_Impl(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath)
		{
			return GetValue<T>(InContainer, InPropertyPath, Property, Value);
		}

		T& Value;
		FProperty*& Property;
	};

	/** Non-UObject helper struct for SetValue function calls */
	template<typename T, typename ContainerType>
	struct FCallSetterFunctionHelper
	{
		static bool CallSetterFunction(ContainerType* InContainer, UFunction* InFunction, const T& InValue)
		{
			// Cant call UFunctions on non-UObject containers
			return false;
		}
	};

	/** UObject partial specialization of FCallSetterFunctionHelper */
	template<typename T>
	struct FCallSetterFunctionHelper<T, UObject>
	{
		static bool CallSetterFunction(UObject* InContainer, UFunction* InFunction, const T& InValue)
		{
			// We only support calling functions that take a single value and return no parameters
			if ( InFunction->NumParms == 1 && InFunction->GetReturnProperty() == nullptr )
			{
				// Verify there's a return property.
				if ( FProperty* ParamProperty = GetFirstParamProperty(InFunction) )
				{
					if constexpr (std::is_same<FPropertyStructView, T>::value)
					{
						const FPropertyStructView& InStuctView = static_cast<const FPropertyStructView&>(InValue);

						// Ensure that the element sizes are the same, prevents the user from doing something terribly wrong.
						if (ParamProperty->GetElementSize() == InStuctView.ScriptStruct->GetStructureSize() && !InContainer->IsUnreachable())
						{
							InContainer->ProcessEvent(InFunction, const_cast<uint8*>(InStuctView.Memory));
							return true;
						}
					}
					// Verify that the cpp type matches a known property type.
					else if ( IsConcreteTypeCompatibleWithReflectedType<T>(ParamProperty) )
					{
						// Ensure that the element sizes are the same, prevents the user from doing something terribly wrong.
						if ( PropertySizesMatch<T>(ParamProperty) && !InContainer->IsUnreachable() )
						{
							InContainer->ProcessEvent(InFunction, const_cast<T*>(&InValue));
							return true;
						}
					}
				}
			}
			else
			{
				//LOG ERROR
			}

			return false;
		}
	};

	/** Helper function used to set a value */
	template<typename T, typename ContainerType>
	struct FSetValueHelper
	{
		static bool SetValue(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath, const T& InValue)
		{
			const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
			int32 ArrayIndex = LastSegment.GetArrayIndex();
			FProperty* Property = CastFieldChecked<FProperty>(LastSegment.GetField().ToField());

			if constexpr (std::is_same<FPropertyStructView, T>::value)
			{
				const FPropertyStructView& InStuctView = static_cast<const FPropertyStructView&>(InValue);

				// Ensure that the element sizes are the same, prevents the user from doing something terribly wrong.
				ArrayIndex = ArrayIndex == INDEX_NONE ? 0 : ArrayIndex;
				if ( Property->GetElementSize() == InStuctView.ScriptStruct->GetStructureSize() && ArrayIndex < Property->ArrayDim)
				{
					if (Property->HasSetter())
					{
						Property->CallSetter(InContainer, InStuctView.Memory);
					}
					else if (void* Address = Property->ContainerPtrToValuePtr<void>(InContainer, ArrayIndex))
					{
						InPropertyPath.ResolveLeaf(Address);
						Property->CopySingleValue(Address, InStuctView.Memory);
						CallParentSetters(InPropertyPath);
					}
					return true;
				}
			}
			// Verify that the cpp type matches a known property type.
			else if ( IsConcreteTypeCompatibleWithReflectedType<T>(Property) )
			{
				// Ensure that the element sizes are the same, prevents the user from doing something terribly wrong.
				ArrayIndex = ArrayIndex == INDEX_NONE ? 0 : ArrayIndex;
				if ( PropertySizesMatch<T>(Property) && ArrayIndex < Property->ArrayDim )
				{
					if (Property->HasSetter())
					{
						Property->CallSetter(InContainer, &InValue);
					}
					else if (void* Address = Property->ContainerPtrToValuePtr<T>(InContainer, ArrayIndex))
					{
						InPropertyPath.ResolveLeaf(Address);
						Property->CopySingleValue(Address, &InValue);
						CallParentSetters(InPropertyPath);
					}
					return true;
				}
			}

			return false;
		}
	};

	/** Partial specialization for arrays */
	template<typename T, typename ContainerType, int32 N>
	struct FSetValueHelper<T[N], ContainerType>
	{
		static bool SetValue(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath, const T(&InValue)[N])
		{
			const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
			FProperty* Property = CastFieldChecked<FProperty>(LastSegment.GetField().ToField());

			// Verify that the cpp type matches a known property type.
			if ( IsConcreteTypeCompatibleWithReflectedType<T>(Property) )
			{
				// Ensure that the element sizes are the same, prevents the user from doing something terribly wrong.
				if ( PropertySizesMatch<T>(Property) && Property->ArrayDim == N )
				{
					if(void* Address = Property->ContainerPtrToValuePtr<T>(InContainer))
					{
						InPropertyPath.ResolveLeaf(Address);
						Property->CopyCompleteValue(Address, &InValue);
						return true;
					}
				}
			}

			return false;
		}
	};

	/** Partial specialization for bools/bitfields */
	template<typename ContainerType>
	struct FSetValueHelper<bool, ContainerType>
	{
		static bool SetValue(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath, const bool& InValue)
		{
			const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
			int32 ArrayIndex = LastSegment.GetArrayIndex();
			FProperty* Property = CastFieldChecked<FProperty>(LastSegment.GetField().ToField());

			// Verify that the cpp type matches a known property type.
			if ( IsConcreteTypeCompatibleWithReflectedType<bool>(Property) )
			{
				// Ensure that the element sizes are the same, prevents the user from doing something terribly wrong.
				ArrayIndex = ArrayIndex == INDEX_NONE ? 0 : ArrayIndex;
				if ( PropertySizesMatch<bool>(Property) && ArrayIndex < Property->ArrayDim )
				{
					if(void* Address = Property->ContainerPtrToValuePtr<bool>(InContainer, ArrayIndex))
					{
						InPropertyPath.ResolveLeaf(Address);

						if (Property->HasSetter())
						{
							// Setter should be specialized to handle masking
							Property->CallSetter(InContainer, &InValue);
						}
						else
						{
							FBoolProperty* BoolProperty = CastFieldChecked<FBoolProperty>(Property);
							BoolProperty->SetPropertyValue(InPropertyPath.GetCachedAddress(), InValue);
							CallParentSetters(InPropertyPath);
						}
						return true;
					}
				}
			}

			return false;
		}
	};

	/** 
	 * Resolve a property path to a property and a value.
	 * @param InContainer			The containing object/structure for the current path iteration
	 * @param InPropertyPath		The property path to set from
	 * @param InValue				The value to set
	 * @return true if the value was resolved
	 * @note This method is specialized internally to work with FPropertyStructView
	 */
	template<typename T, typename ContainerType>
	bool SetValue(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath, const T& InValue)
	{
		const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
		int32 ArrayIndex = LastSegment.GetArrayIndex();
		FFieldVariant Field = LastSegment.GetField();

		// We're on the final property in the path, it may be an array property, so check that first.
		if ( FArrayProperty* ArrayProp = Field.Get<FArrayProperty>() )
		{
			// if it's an array property, we need to see if we parsed an index as part of the segment
			// as a user may have baked the index directly into the property path.
			if(ArrayIndex != INDEX_NONE)
			{
				// Property is an array property, so make sure we have a valid index specified
				FScriptArrayHelper_InContainer ArrayHelper(ArrayProp, InContainer);
				if ( ArrayHelper.IsValidIndex(ArrayIndex) )
				{
					if constexpr (std::is_same<FPropertyStructView, T>::value)
					{
						const FPropertyStructView& InStuctView = static_cast<const FPropertyStructView&>(InValue);

						// Ensure that the element sizes are the same, prevents the user from doing something terribly wrong.
						if (ArrayProp->Inner->GetElementSize() == InStuctView.ScriptStruct->GetStructureSize())
						{
							if (void* Address = static_cast<void*>(ArrayHelper.GetRawPtr(ArrayIndex)))
							{
								InPropertyPath.ResolveLeaf(Address);
								ArrayProp->Inner->CopySingleValue(Address, InStuctView.Memory);
								return true;
							}
						}
					}
					else
					{
						// Verify that the cpp type matches a known property type.
						if (IsConcreteTypeCompatibleWithReflectedType<T>(ArrayProp->Inner))
						{
							// Ensure that the element sizes are the same, prevents the user from doing something terribly wrong.
							if (PropertySizesMatch<T>(ArrayProp->Inner))
							{
								if (void* Address = static_cast<void*>(ArrayHelper.GetRawPtr(ArrayIndex)))
								{
									InPropertyPath.ResolveLeaf(Address);
									ArrayProp->Inner->CopySingleValue(Address, &InValue);
									return true;
								}
							}
						}
					}
				}
			}
			else
			{
				// No index, so assume we want the array property itself
				// Verify that the cpp type matches a known property type.
				if constexpr (!std::is_same<FPropertyStructView, T>::value)
				{
					if (IsConcreteTypeCompatibleWithReflectedType<T>(ArrayProp))
					{
						// Ensure that the element sizes are the same, prevents the user from doing something terribly wrong.
						if (PropertySizesMatch<T>(ArrayProp))
						{
							if (void* Address = ArrayProp->ContainerPtrToValuePtr<T>(InContainer))
							{
								InPropertyPath.ResolveLeaf(Address);
								ArrayProp->CopySingleValue(Address, &InValue);
								return true;
							}
						}
					}
				}
			}
		}
		else if(UFunction* Function = Field.Get<UFunction>())
		{
			InPropertyPath.ResolveLeaf(Function);
			return FCallSetterFunctionHelper<T, ContainerType>::CallSetterFunction(InContainer, Function, InValue);
		}
		else if(FProperty* Property = Field.Get<FProperty>())
		{
			return FSetValueHelper<T, ContainerType>::SetValue(InContainer, InPropertyPath, InValue);
		}

		return false;
	}

	template<typename T>
	struct FInternalSetterResolver : public TPropertyPathResolver<FInternalSetterResolver<T>>
	{
		FInternalSetterResolver(const T& InValueToSet)
			: Value(InValueToSet)
		{
		}

		template<typename ContainerType>
		bool Resolve_Impl(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath)
		{
			return SetValue<T>(InContainer, InPropertyPath, Value);
		}

		const T& Value;
	};

	/** 
	 * Resolve a property path using the specified resolver 
	 * @param	InContainer		The containing object to iterate against
	 * @param	InPropertyPath	The property path to iterate
	 * @param	InResolver		The visitor to use
	 * @return true if the iteration completed successfully
	 */
	PROPERTYPATH_API bool ResolvePropertyPath(UObject* InContainer, const FCachedPropertyPath& InPropertyPath, FPropertyPathResolver& InResolver);

	/** 
	 * Resolve a property path using the specified resolver 
	 * @param	InContainer		The containing object/structure to iterate against
	 * @param	InStruct		The structure type that InContainer points to
	 * @param	InPropertyPath	The property path to iterate
	 * @param	InResolver		The visitor to use
	 * @return true if the iteration completed successfully
	 */
	PROPERTYPATH_API bool ResolvePropertyPath(void* InContainer, UStruct* InStruct, const FCachedPropertyPath& InPropertyPath, FPropertyPathResolver& InResolver);

	/** 
	 * Resolve a property path using the specified resolver 
	 * @param	InContainer		The containing object to iterate against
	 * @param	InPropertyPath	The property path to iterate
	 * @param	InResolver		The resolver to use
	 * @return true if the iteration completed successfully
	 */
	PROPERTYPATH_API bool ResolvePropertyPath(UObject* InContainer, const FString& InPropertyPath, FPropertyPathResolver& InResolver);

	/** 
	 * Resolve a property path using the specified resolver 
	 * @param	InContainer		The containing object/structure to iterate against
	 * @param	InStruct		The structure type that InContainer points to
	 * @param	InPropertyPath	The property path to iterate
	 * @param	InResolver		The resolver to use
	 * @return true if the iteration completed successfully
	 */
	PROPERTYPATH_API bool ResolvePropertyPath(void* InContainer, UStruct* InStruct, const FString& InPropertyPath, FPropertyPathResolver& InResolver);

	/** Helper function used to get a value from an already-resolved property path */
	template<typename T>
	struct FGetValueFastHelper
	{
		static bool GetValue(const FCachedPropertyPath& InPropertyPath, T& OutValue, FProperty*& OutProperty)
		{
			const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
			OutProperty = CastFieldChecked<FProperty>(LastSegment.GetField().ToField());
			FArrayProperty* ArrayProp = CastField<FArrayProperty>(OutProperty);
			if ( ArrayProp && LastSegment.GetArrayIndex() != INDEX_NONE )
			{
				if (IsConcreteTypeCompatibleWithReflectedType<T>(ArrayProp->Inner))
				{
					ArrayProp->Inner->CopySingleValue(&OutValue, InPropertyPath.GetCachedAddress());
					return true;
				}
			}
			else if (IsConcreteTypeCompatibleWithReflectedType<T>(OutProperty))
			{
				if (OutProperty->HasGetter())
				{
					OutProperty->CallGetter(InPropertyPath.GetCachedContainer(), &OutValue);
				}
				else
				{
					CallParentGetters(&OutValue, InPropertyPath, InPropertyPath.GetCachedAddress());
				}	
				return true;
			}
			return false;
		}
	};

	/** Partial specialization for arrays */
	template<typename T, int32 N>
	struct FGetValueFastHelper<T[N]>
	{
		static bool GetValue(const FCachedPropertyPath& InPropertyPath, T(&OutValue)[N], FProperty*& OutProperty)
		{
			const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
			OutProperty = CastFieldChecked<FProperty>(LastSegment.GetField().ToField());
			if (IsConcreteTypeCompatibleWithReflectedType<T>(OutProperty))
			{
				OutProperty->CopyCompleteValue(&OutValue, InPropertyPath.GetCachedAddress());
				return true;
			}
			return false;
		}
	};

	/** 
	 * Fast, unsafe version of GetValue().
	 * @param	InContainer		The containing object/structure to iterate against
	 * @param	InPropertyPath	The property path to use
	 * @param	OutValue		The value to write to
	 * @param	OutProperty		The leaf property that the path resolved to
	 * @return true if the value was written successfully
	 */
	template<typename T, typename ContainerType>
	bool GetValueFast(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath, T& OutValue, FProperty*& OutProperty)
	{
		if(InPropertyPath.GetCachedFunction())
		{
			return FCallGetterFunctionHelper<T, ContainerType>::CallGetterFunction(InContainer, InPropertyPath.GetCachedFunction(), OutValue);
		}
		else if(InPropertyPath.GetCachedAddress())
		{
			return FGetValueFastHelper<T>::GetValue(InPropertyPath, OutValue, OutProperty);
		}

		return false;
	}

	/** Helper function used to set a value from an already-resolved property path */
	template<typename T>
	struct FSetValueFastHelper
	{
		static bool SetValue(const FCachedPropertyPath& InPropertyPath, const T& InValue)
		{
			const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
			FProperty* Property = CastFieldChecked<FProperty>(LastSegment.GetField().ToField());
			FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property);

			const void* Value = nullptr;
			if constexpr (std::is_same<FPropertyStructView, T>::value)
			{
				const FPropertyStructView& InStuctView = static_cast<const FPropertyStructView&>(InValue);
				Value = static_cast<const void*>(InStuctView.Memory);
			}
			else
			{
				Value = &InValue;
			}

			auto IsPropertyCompatible = [](FProperty* InProperty)
			{
				if constexpr (!std::is_same<FPropertyStructView, T>::value)
				{
					if (!IsConcreteTypeCompatibleWithReflectedType<T>(InProperty))
					{
						return false;
					}
				}

				return true;
			};

			if ( ArrayProp && LastSegment.GetArrayIndex() != INDEX_NONE )
			{
				if (!IsPropertyCompatible(ArrayProp->Inner))
				{
					return false;
				}

				ArrayProp->Inner->CopySingleValue(InPropertyPath.GetCachedAddress(), Value);
				return true;
			}
			else 
			{
				if (!IsPropertyCompatible(Property))
				{
					return false;
				}

				if (Property->HasSetter())
				{
					Property->CallSetter(InPropertyPath.GetCachedContainer(), Value);
				}
				else
				{
					Property->CopySingleValue(InPropertyPath.GetCachedAddress(), Value);
					CallParentSetters(InPropertyPath);
				}
				return true;
			}
		}
	};

	/** Partial specialization for arrays */
	template<typename T, int32 N>
	struct FSetValueFastHelper<T[N]>
	{
		static bool SetValue(const FCachedPropertyPath& InPropertyPath, const T(&InValue)[N])
		{
			const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
			FProperty* Property = CastFieldChecked<FProperty>(LastSegment.GetField().ToField());
			if (IsConcreteTypeCompatibleWithReflectedType<T>(Property))
			{
				Property->CopyCompleteValue(InPropertyPath.GetCachedAddress(), &InValue);
				return true;
			}
			return false;
		}
	};

	/** Explicit specialization for bools/bitfields */
	template<>
	struct FSetValueFastHelper<bool>
	{
		static bool SetValue(const FCachedPropertyPath& InPropertyPath, const bool& InValue)
		{
			const FPropertyPathSegment& LastSegment = InPropertyPath.GetLastSegment();
			FProperty* Property = CastFieldChecked<FProperty>(LastSegment.GetField().ToField());
			FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property);
			if ( ArrayProp && LastSegment.GetArrayIndex() != INDEX_NONE )
			{
				if (IsConcreteTypeCompatibleWithReflectedType<bool>(ArrayProp->Inner))
				{
					ArrayProp->Inner->CopySingleValue(InPropertyPath.GetCachedAddress(), &InValue);
					return true;
				}
			}
			else if (IsConcreteTypeCompatibleWithReflectedType<bool>(Property))
			{
				if (Property->HasSetter())
				{
					// Setter should be specialized to handle masking
					Property->CallSetter(InPropertyPath.GetCachedContainer(), &InValue);
				}
				else
				{
					FBoolProperty* BoolProperty = CastFieldChecked<FBoolProperty>(Property);
					BoolProperty->SetPropertyValue(InPropertyPath.GetCachedAddress(), InValue);
					CallParentSetters(InPropertyPath);
				}
				return true;
			}
			return false;
		}
	};

	/** 
	 * Fast, unsafe version of SetValue().
	 * @param	InContainer		The containing object/structure to iterate against
	 * @param	InPropertyPath	The property path to use
	 * @param	OutValue		The value to read from
	 * @return true if the value was read successfully
	 */
	template<typename T, typename ContainerType>
	bool SetValueFast(ContainerType* InContainer, const FCachedPropertyPath& InPropertyPath, const T& InValue)
	{
		if(InPropertyPath.GetCachedFunction())
		{
			return FCallSetterFunctionHelper<T, ContainerType>::CallSetterFunction(InContainer, InPropertyPath.GetCachedFunction(), InValue);
		}
		else if(InPropertyPath.GetCachedAddress())
		{
			return FSetValueFastHelper<T>::SetValue(InPropertyPath, InValue);
		}

		return false;
	}
}
