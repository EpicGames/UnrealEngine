// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Traits/IsTEnumAsByte.h"
#include "UObject/Class.h"
#include "UObject/StrProperty.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/UnrealType.h"
#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"
#include "Math/InterpCurvePoint.h"

/** Generic (struct) implementation */
template<typename T>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl(FProperty* Property)
{
	if ( FStructProperty* StructProperty = CastField<FStructProperty>(Property) )
	{
		return StructProperty->Struct == T::StaticStruct();
	}

	return false;
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<bool>(FProperty* Property)
{
	return Property->GetClass() == FBoolProperty::StaticClass();
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<int8>(FProperty* Property)
{
	if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		Property = EnumProperty->GetUnderlyingProperty();
	}

	return Property->GetClass() == FInt8Property::StaticClass();
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<uint8>(FProperty* Property)
{
	if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		Property = EnumProperty->GetUnderlyingProperty();
	}
	else if(FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
	{
		return !BoolProperty->IsNativeBool();
	}

	return Property->GetClass() == FByteProperty::StaticClass();
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<int16>(FProperty* Property)
{
	if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		Property = EnumProperty->GetUnderlyingProperty();
	}

	return Property->GetClass() == FInt16Property::StaticClass();
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<uint16>(FProperty* Property)
{
	if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		Property = EnumProperty->GetUnderlyingProperty();
	}
	else if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
	{
		return !BoolProperty->IsNativeBool();
	}

	return Property->GetClass() == FUInt16Property::StaticClass();
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<int32>(FProperty* Property)
{
	if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		Property = EnumProperty->GetUnderlyingProperty();
	}

	return Property->GetClass() == FIntProperty::StaticClass();
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<uint32>(FProperty* Property)
{
	if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		Property = EnumProperty->GetUnderlyingProperty();
	}
	else if(FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
	{
		return !BoolProperty->IsNativeBool();
	}

	return Property->GetClass() == FUInt32Property::StaticClass();
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<int64>(FProperty* Property)
{
	if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		Property = EnumProperty->GetUnderlyingProperty();
	}

	return Property->GetClass() == FInt64Property::StaticClass();
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<uint64>(FProperty* Property)
{
	if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		Property = EnumProperty->GetUnderlyingProperty();
	}
	else if(FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
	{
		return !BoolProperty->IsNativeBool();
	}

	return Property->GetClass() == FUInt64Property::StaticClass();
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<float>(FProperty* Property)
{
	return Property->GetClass() == FFloatProperty::StaticClass();
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<double>(FProperty* Property)
{
	return Property->GetClass() == FDoubleProperty::StaticClass();
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FText>(FProperty* Property)
{
	return Property->GetClass() == FTextProperty::StaticClass();
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FString>(FProperty* Property)
{
	return Property->GetClass() == FStrProperty::StaticClass();
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FName>(FProperty* Property)
{
	return Property->GetClass() == FNameProperty::StaticClass();
}

template<typename T>
inline bool IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct(FProperty* Property)
{
	static const UScriptStruct* BuiltInStruct = TBaseStructure<T>::Get();
	
	if ( FStructProperty* StructProperty = CastField<FStructProperty>(Property) )
	{
		return StructProperty->Struct == BuiltInStruct;
	}

	return false;
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FColor>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FColor>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FLinearColor>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FLinearColor>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FVector2D>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FVector2D>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FVector>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FVector>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FRotator>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FRotator>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FQuat>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FQuat>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FTransform>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FTransform>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FBox2D>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FBox2D>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FGuid>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FGuid>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FInterpCurvePointFloat>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FInterpCurvePointFloat>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FInterpCurvePointVector>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FInterpCurvePointVector>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FInterpCurvePointVector2D>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FInterpCurvePointVector2D>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FInterpCurvePointQuat>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FInterpCurvePointQuat>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FInterpCurvePointTwoVectors>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FInterpCurvePointTwoVectors>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FInterpCurvePointLinearColor>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FInterpCurvePointLinearColor>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FFloatRangeBound>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FFloatRangeBound>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FFloatRange>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FFloatRange>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FDoubleInterval>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FDoubleInterval>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FInt32RangeBound>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FInt32RangeBound>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FInt32Range>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FInt32Range>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FFloatInterval>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FFloatInterval>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FInt32Interval>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FInt32Interval>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FSoftObjectPath>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FSoftObjectPath>(Property);
}

template<>
inline bool IsConcreteTypeCompatibleWithReflectedType_Impl<FSoftClassPath>(FProperty* Property)
{
	return IsConcreteTypeCompatibleWithReflectedType_BuiltInStruct<FSoftClassPath>(Property);
}

/** Standard implementation */
template<typename T> 
struct FConcreteTypeCompatibleWithReflectedTypeHelper
{
	static bool IsConcreteTypeCompatibleWithReflectedType(FProperty* Property) 
	{
		if constexpr (TIsPointerOrObjectPtrToBaseOf<T, UObject>::Value)
		{
			if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
			{
				return true;
				//return ObjectProperty->PropertyClass->IsChildOf(T::StaticClass());
			}

			return false;
		}
		else
		{
			return IsConcreteTypeCompatibleWithReflectedType_Impl<T>(Property);
		}
	}
};

/** Dynamic array partial specialization */
template<typename T>
struct FConcreteTypeCompatibleWithReflectedTypeHelper<TArray<T>>
{
	static bool IsConcreteTypeCompatibleWithReflectedType(FProperty* Property) 
	{
		if( FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property) )
		{
			return IsConcreteTypeCompatibleWithReflectedType_Impl<T>(ArrayProperty->Inner);
		}

		return false;
	}
};

/** Static array partial specialization */
template<typename T, int32 N>
struct FConcreteTypeCompatibleWithReflectedTypeHelper<T[N]>
{
	static bool IsConcreteTypeCompatibleWithReflectedType(FProperty* Property) 
	{
		return Property->ArrayDim == N && IsConcreteTypeCompatibleWithReflectedType_Impl<T>(Property);
	}
};

/** Weak object partial specialization */
template<typename T>
struct FConcreteTypeCompatibleWithReflectedTypeHelper<TWeakObjectPtr<T>>
{
	static bool IsConcreteTypeCompatibleWithReflectedType(FProperty* Property) 
	{
		return Property->GetClass() == FWeakObjectProperty::StaticClass();
	}
};

/** Weak object partial specialization */
template<typename T>
struct FConcreteTypeCompatibleWithReflectedTypeHelper<TLazyObjectPtr<T>>
{
	static bool IsConcreteTypeCompatibleWithReflectedType(FProperty* Property) 
	{
		return Property->GetClass() == FLazyObjectProperty::StaticClass();
	}
};

/** 
 * Check whether the concrete type T is compatible with the reflected type of a FProperty for 
 * the purposes of CopysingleValue()
 */
template<typename T>
static inline bool IsConcreteTypeCompatibleWithReflectedType(FProperty* Property)
{
	if constexpr (TIsEnum<T>::Value || TIsTEnumAsByte_V<T>)
	{
		return FConcreteTypeCompatibleWithReflectedTypeHelper<uint8>::IsConcreteTypeCompatibleWithReflectedType(Property);
	}
	else
	{
		return FConcreteTypeCompatibleWithReflectedTypeHelper<T>::IsConcreteTypeCompatibleWithReflectedType(Property);
	}
}

template<typename T>
inline bool PropertySizesMatch_Impl(FProperty* InProperty)
{
	if (FByteProperty* ByteProperty = CastField<FByteProperty>(InProperty))
	{
		return InProperty->GetElementSize() == sizeof(uint8);
	}

	return InProperty->GetElementSize() == sizeof(T);
}

template<>
inline bool PropertySizesMatch_Impl<uint8>(FProperty* InProperty)
{
	if(FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
	{
		return !BoolProperty->IsNativeBool();
	}

	return InProperty->GetElementSize() == sizeof(uint8);
}

template<>
inline bool PropertySizesMatch_Impl<uint16>(FProperty* InProperty)
{
	if(FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
	{
		return !BoolProperty->IsNativeBool();
	}

	return InProperty->GetElementSize() == sizeof(uint16);
}

template<>
inline bool PropertySizesMatch_Impl<uint32>(FProperty* InProperty)
{
	if(FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
	{
		return !BoolProperty->IsNativeBool();
	}

	return InProperty->GetElementSize() == sizeof(uint32);
}

template<>
inline bool PropertySizesMatch_Impl<uint64>(FProperty* InProperty)
{
	if(FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
	{
		return !BoolProperty->IsNativeBool();
	}

	return InProperty->GetElementSize() == sizeof(uint64);
}

template<typename T>
struct FPropertySizesMatchHelper
{
	static bool PropertySizesMatch(FProperty* InProperty)
	{
		return PropertySizesMatch_Impl<T>(InProperty);
	}
};

template<typename T, int32 N>
struct FPropertySizesMatchHelper<T[N]>
{
	static bool PropertySizesMatch(FProperty* InProperty)
	{
		return PropertySizesMatch_Impl<T>(InProperty);
	}
};

template<typename T>
inline bool PropertySizesMatch(FProperty* InProperty)
{
	return FPropertySizesMatchHelper<T>::PropertySizesMatch(InProperty);
}
